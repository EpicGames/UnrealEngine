{
	"FileVersion" : 1,
	
	"FriendlyName" : "Online Subsystem Tencent",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Access to Tencent platform",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,

	"Modules": [
		{
			"Name": "OnlineSubsystemTencent",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": 
			[
				"Linux",
				"Win64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystemUtils",
			"Enabled": true
		},
		{
			"Name": "OnlineFramework",
			"Enabled": true
		}
	]
}