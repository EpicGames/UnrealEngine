// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class OnlineSubsystemTencent : ModuleRules
{
	public OnlineSubsystemTencent(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDefinitions.Add("ONLINESUBSYSTEM_TENCENT_PACKAGE=1");

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Sockets",
				"HTTP",
				"Json",
				"OnlineSubsystem",
				"PacketHandler",
				"PlayTimeLimit"
			}
		);

		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Engine",
					"OnlineSubsystemUtils"
				}
			);
		}

		if (System.IO.Directory.Exists(System.IO.Path.Combine(Target.UEThirdPartySourceDirectory, "Tencent", "WeGame")))
		{
			AddEngineThirdPartyPrivateDynamicDependencies(Target, "WeGame");
			PublicIncludePathModuleNames.Add("WeGame");
			PublicDefinitions.Add("WITH_TENCENTSDK=1");
		}
		else
		{
			PublicDefinitions.Add("WITH_TENCENTSDK=0");
		}

		if (System.IO.Directory.Exists(System.IO.Path.Combine(EngineDirectory, "Restricted/NotForLicensees/Source/ThirdParty/Tencent")))
		{
			AddEngineThirdPartyPrivateDynamicDependencies(Target, "Tencent");
			PublicIncludePathModuleNames.Add("Tencent");
		}

		// Don't want to introduce Tencent dependency when building the base editor
		PrecompileForTargets = PrecompileTargetsType.None;
	}
}
