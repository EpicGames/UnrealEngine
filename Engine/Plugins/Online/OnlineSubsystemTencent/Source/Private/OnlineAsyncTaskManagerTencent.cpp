// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAsyncTaskManagerTencent.h"

#if WITH_TENCENTSDK

#if WITH_TENCENT_RAIL_SDK
#include "Features/IModularFeatures.h"
#include "OnlineSubsystemTencentPrivate.h"
#include "OnlineAsyncTasksTencent.h"
#include "OnlinePresenceTencent.h"
#include "PlayTimeLimitImpl.h"
#include "RailSdkWrapper.h"
#endif

FOnlineAsyncTaskManagerTencent::FOnlineAsyncTaskManagerTencent()
	: Subsystem(nullptr)
{
#if WITH_TENCENT_RAIL_SDK
	RegisterRailEvents();
#endif
}


FOnlineAsyncTaskManagerTencent::FOnlineAsyncTaskManagerTencent(class FOnlineSubsystemTencent* InSubsystem)
	: Subsystem(InSubsystem)
{
#if WITH_TENCENT_RAIL_SDK
	RegisterRailEvents();
#endif
}

FOnlineAsyncTaskManagerTencent::~FOnlineAsyncTaskManagerTencent()
{
#if WITH_TENCENT_RAIL_SDK
	UnregisterRailEvents();
#endif
}

void FOnlineAsyncTaskManagerTencent::OnlineTick()
{
	check(Subsystem);
	check(FPlatformTLS::GetCurrentThreadId() == OnlineThreadId || !FPlatformProcess::SupportsMultithreading());

#if WITH_TENCENT_RAIL_SDK
	RailSdkWrapper& RailSDK = RailSdkWrapper::Get();
	if (RailSDK.IsInitialized())
	{
		RailSDK.RailFireEvents();
	}
#endif
}

#if WITH_TENCENT_RAIL_SDK

void FOnlineAsyncTaskManagerTencent::RegisterRailEvents()
{
	RailSdkWrapper& RailSDK = RailSdkWrapper::Get();
	if (RailSDK.IsInitialized())
	{
		/**
		 * System Events
		 */

		// Rail system state changed (for example launcher was closed) (RailSystemStateChanged)
		RegisteredRailEvents.Add(rail::kRailEventSystemStateChanged);

		/**
		 * Friends
		 */

		// Friend add request has completed (RailFriendsAddFriendResult)
		RegisteredRailEvents.Add(rail::kRailEventFriendsAddFriendResult);
		// Friends list has changed (RailFriendsListChanged)
		RegisteredRailEvents.Add(rail::kRailEventFriendsFriendsListChanged);
		// Friend's online status has changed (RailFriendsOnlineStateChanged)
		RegisteredRailEvents.Add(rail::kRailEventFriendsOnlineStateChanged);
		// Friends' key/value pairs metadata has changed (RailFriendsMetadataChanged)
		RegisteredRailEvents.Add(rail::kRailEventFriendsMetadataChanged);
		
		/**
		 * External UI
		 */

		// Floating window dialog has opened/closed (ShowFloatingWindowResult)
		RegisteredRailEvents.Add(rail::kRailEventShowFloatingWindow);
		// Floating notify window dialog should be displayed by application
		RegisteredRailEvents.Add(rail::kRailEventShowFloatingNotifyWindow);

		/**
		 * Invites
		 */

		// Invite has been sent to a remote user notification (RailUsersNotifyInviter)
		RegisteredRailEvents.Add(rail::kRailEventUsersNotifyInviter);
		// Invite has been sent to a remote user via AsyncInviteUsers (RailUsersInviteUsersResult)
		RegisteredRailEvents.Add(rail::kRailEventUsersInviteUsersResult);
		// Inviter receives a response regarding invitee handling invite (RailUsersInviteJoinGameResult)
		RegisteredRailEvents.Add(rail::kRailEventUsersInviteJoinGameResult);
		// Invitee has responded to inviter's invite (RailUsersRespondInvitation)
		RegisteredRailEvents.Add(rail::kRailEventUsersRespondInvitation);
		// Join via presence user trying to join a remote session (RailPlatformNotifyEventJoinGameByUser)
		RegisteredRailEvents.Add(rail::kRailPlatformNotifyEventJoinGameByUser);

		/**
		 * Assets
		 */
		// Assets in the user's inventory has changed (RailAssetsChanged)
		RegisteredRailEvents.Add(rail::kRailEventAssetsAssetsChanged);
		
		/**
		 * NYI
		 */

		// Querying a friends running game is complete (RailFriendsQueryFriendPlayedGamesResult)
		RegisteredRailEvents.Add(rail::kRailEventFriendsGetFriendPlayedGamesResult);
		// Querying a user's info is complete (RailUsersInfoData)
		RegisteredRailEvents.Add(rail::kRailEventUsersGetUsersInfo);
		// (RailUsersCancelInviteResult)
		RegisteredRailEvents.Add(rail::kRailEventUsersCancelInviteResult);
		RegisteredRailEvents.Add(rail::kRailEventFriendsDialogShow);

		/** 
		 * Implemented by Async Tasks
		 */
		RegisteredRailEvents.Add(rail::kRailEventFriendsSetMetadataResult);
		RegisteredRailEvents.Add(rail::kRailEventFriendsGetInviteCommandLine);

		for (auto EventId : RegisteredRailEvents)
		{
			RailSDK.RailRegisterEvent(EventId, this);
		}
	}
}

void FOnlineAsyncTaskManagerTencent::UnregisterRailEvents()
{
	RailSdkWrapper& RailSDK = RailSdkWrapper::Get();
	if (RailSDK.IsInitialized())
	{
		for (auto EventId : RegisteredRailEvents)
		{
			RailSDK.RailUnregisterEvent(EventId, this);
		}
		RegisteredRailEvents.Empty();
	}
}

void FOnlineAsyncTaskManagerTencent::OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param)
{
	UE_LOG_ONLINE(Verbose, TEXT("FOnlineAsyncTaskManagerTencent::OnRailEvent EventId: %d Result: %d"), static_cast<uint32>(event_id), param ? param->get_result() : -1);

	/** THIS IS ONLINE THREAD */
	if ((OnlineThreadId != 0) && (FPlatformTLS::GetCurrentThreadId() != OnlineThreadId))
	{
		UE_LOG_ONLINE(Warning, TEXT("Task ran on thread other than online!"));
	}
	
	switch (event_id)
	{
		case rail::kRailEventSystemStateChanged:
			OnRailSystemStateChanged(static_cast<rail::rail_event::RailSystemStateChanged*>(param));
			break;
		case rail::kRailEventShowFloatingWindow:
			OnRailShowFloatingWindow(static_cast<rail::rail_event::ShowFloatingWindowResult*>(param));
			break;
		case rail::kRailEventShowFloatingNotifyWindow:
			OnRailShowFloatingNotifyWindow(static_cast<rail::rail_event::ShowNotifyWindow*>(param));
			break;
		case rail::kRailEventFriendsDialogShow:
			break;
		case rail::kRailEventUsersNotifyInviter:
			OnRailInviteSentNotification(static_cast<rail::rail_event::RailUsersNotifyInviter*>(param));
			break;
		case rail::kRailEventUsersInviteUsersResult:
			OnRailInviteUsersResult(static_cast<rail::rail_event::RailUsersInviteUsersResult*>(param));
			break;
		case rail::kRailEventUsersInviteJoinGameResult:
			OnRailInviteJoinGameResult(static_cast<rail::rail_event::RailUsersInviteJoinGameResult*>(param));
			break;
		case rail::kRailEventUsersRespondInvitation:
			OnRailFriendsInviteRespondInvitation(static_cast<rail::rail_event::RailUsersRespondInvitation*>(param));
			break;
		case rail::kRailPlatformNotifyEventJoinGameByUser:
			OnRailFriendsJoinGameByUser(static_cast<rail::rail_event::RailPlatformNotifyEventJoinGameByUser*>(param));
			break;
		case rail::kRailEventAssetsAssetsChanged:
			OnRailAssetsChanged(static_cast<rail::rail_event::RailAssetsChanged*>(param));
			break;
		case rail::kRailEventUsersGetUsersInfo:
			OnRailGetUsersInfo(static_cast<rail::rail_event::RailUsersInfoData*>(param));
			break;
		case rail::kRailEventFriendsGetFriendPlayedGamesResult:
			OnRailGetFriendPlayedGamesResult(static_cast<rail::rail_event::RailFriendsQueryFriendPlayedGamesResult*>(param));
			break;
		case rail::kRailEventFriendsGetInviteCommandLine:
			OnRailInviteCommandline(static_cast<rail::rail_event::RailFriendsGetInviteCommandLine*>(param));
			break;
		case rail::kRailEventUsersCancelInviteResult:
			OnRailCancelInviteResult(static_cast<rail::rail_event::RailUsersCancelInviteResult*>(param));
			break;
		case rail::kRailEventFriendsSetMetadataResult:
			OnRailFriendsSetMetadataResult(static_cast<rail::rail_event::RailFriendsSetMetadataResult*>(param));
			break;
		case rail::kRailEventFriendsAddFriendResult:
			OnRailEventFriendsAddFriendResult(static_cast<rail::rail_event::RailFriendsAddFriendResult*>(param));
			break;
		case rail::kRailEventFriendsFriendsListChanged:
			OnRailFriendsListChanged(static_cast<const rail::rail_event::RailFriendsListChanged*>(param));
			break;
		case rail::kRailEventFriendsOnlineStateChanged:
			OnRailFriendsOnlineStateChanged(static_cast<const rail::rail_event::RailFriendsOnlineStateChanged*>(param));
			break;
		case rail::kRailEventFriendsMetadataChanged:
			OnRailFriendsMetadataChanged(static_cast<const rail::rail_event::RailFriendsMetadataChanged*>(param));
			break;
		default:
			UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskManagerTencent::OnRailEvent Unhandled Event"));
			break;
	}
}

void FOnlineAsyncTaskManagerTencent::OnRailSystemStateChanged(const rail::rail_event::RailSystemStateChanged* param)
{
	FOnlineAsyncEventRailSystemStateChanged* NewEvent = new FOnlineAsyncEventRailSystemStateChanged(Subsystem, param);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

void FOnlineAsyncTaskManagerTencent::OnRailShowFloatingWindow(const rail::rail_event::ShowFloatingWindowResult* param)
{
	FOnlineAsyncEventRailShowFloatingWindow* NewEvent = new FOnlineAsyncEventRailShowFloatingWindow(Subsystem, param);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

void FOnlineAsyncTaskManagerTencent::OnRailShowFloatingNotifyWindow(const rail::rail_event::ShowNotifyWindow* param)
{
	if (IModularFeatures::Get().IsModularFeatureAvailable(IOnlinePlayTimeLimit::GetModularFeatureName()))
	{
		FOnlineAsyncEventRailShowFloatingNotifyWindow* NewEvent = new FOnlineAsyncEventRailShowFloatingNotifyWindow(Subsystem, param);
		UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
		AddToOutQueue(NewEvent);
	}
}

void FOnlineAsyncTaskManagerTencent::OnRailInviteSentNotification(const rail::rail_event::RailUsersNotifyInviter* param)
{
	FOnlineAsyncEventRailInviteSent* NewEvent = new FOnlineAsyncEventRailInviteSent(Subsystem, param);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

void FOnlineAsyncTaskManagerTencent::OnRailInviteUsersResult(const rail::rail_event::RailUsersInviteUsersResult* param)
{
	FOnlineAsyncEventRailInviteSentEx* NewEvent = new FOnlineAsyncEventRailInviteSentEx(Subsystem, param);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

void FOnlineAsyncTaskManagerTencent::OnRailInviteJoinGameResult(const rail::rail_event::RailUsersInviteJoinGameResult* param)
{
	FOnlineAsyncEventRailJoinGameResult* NewEvent = new FOnlineAsyncEventRailJoinGameResult(Subsystem, param);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

void FOnlineAsyncTaskManagerTencent::OnRailFriendsInviteRespondInvitation(const rail::rail_event::RailUsersRespondInvitation* param)
{
	FOnlineAsyncEventRailInviteResponse* NewEvent = new FOnlineAsyncEventRailInviteResponse(Subsystem, param);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

void FOnlineAsyncTaskManagerTencent::OnRailFriendsJoinGameByUser(const rail::rail_event::RailPlatformNotifyEventJoinGameByUser* param)
{
	FOnlineAsyncEventRailJoinGameByUser* NewEvent = new FOnlineAsyncEventRailJoinGameByUser(Subsystem, param);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

void FOnlineAsyncTaskManagerTencent::OnRailAssetsChanged(const rail::rail_event::RailAssetsChanged* param)
{
	FOnlineAsyncEventRailAssetsChanged* NewEvent = new FOnlineAsyncEventRailAssetsChanged(Subsystem, param);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

void FOnlineAsyncTaskManagerTencent::OnRailGetUsersInfo(const rail::rail_event::RailUsersInfoData* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManagerTencent::OnRailGetUsersInfo NOOP %s"), *LexToString(param ? param->get_result() : rail::RailResult::kErrorUnknown));
}

void FOnlineAsyncTaskManagerTencent::OnRailGetFriendPlayedGamesResult(const rail::rail_event::RailFriendsQueryFriendPlayedGamesResult* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManagerTencent::OnRailGetFriendPlayedGamesResult NOOP %s"), *LexToString(param ? param->get_result() : rail::RailResult::kErrorUnknown));
}

void FOnlineAsyncTaskManagerTencent::OnRailInviteCommandline(const rail::rail_event::RailFriendsGetInviteCommandLine* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManagerTencent::OnRailInviteCommandline NOOP %s"), *LexToString(param ? param->get_result() : rail::RailResult::kErrorUnknown));
}

void FOnlineAsyncTaskManagerTencent::OnRailCancelInviteResult(const rail::rail_event::RailUsersCancelInviteResult* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManagerTencent::OnRailCancelInviteResult NOOP %s"), *LexToString(param ? param->get_result() : rail::RailResult::kErrorUnknown));
}

void FOnlineAsyncTaskManagerTencent::OnRailFriendsSetMetadataResult(const rail::rail_event::RailFriendsSetMetadataResult* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManagerTencent::OnRailFriendsSetMetadataResult NOOP %s"), *LexToString(param ? param->get_result() : rail::RailResult::kErrorUnknown));
}

void FOnlineAsyncTaskManagerTencent::OnRailEventFriendsAddFriendResult(const rail::rail_event::RailFriendsAddFriendResult* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskManagerTencent::OnRailEventFriendsAddFriendResult NOOP %s"), *LexToString(param ? param->get_result() : rail::RailResult::kErrorUnknown));
}

void FOnlineAsyncTaskManagerTencent::OnRailFriendsListChanged(const rail::rail_event::RailFriendsListChanged* param)
{
	FOnlineAsyncEventRailFriendsListChanged* NewEvent = new FOnlineAsyncEventRailFriendsListChanged(Subsystem, param);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

void FOnlineAsyncTaskManagerTencent::OnRailFriendsOnlineStateChanged(const rail::rail_event::RailFriendsOnlineStateChanged* param)
{
	FOnlineAsyncEventRailFriendsOnlineStateChanged* NewEvent = new FOnlineAsyncEventRailFriendsOnlineStateChanged(Subsystem, param);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

void FOnlineAsyncTaskManagerTencent::OnRailFriendsMetadataChanged(const rail::rail_event::RailFriendsMetadataChanged* param)
{
	FOnlineAsyncEventRailFriendsMetadataChanged* NewEvent = new FOnlineAsyncEventRailFriendsMetadataChanged(Subsystem, param);
	UE_LOG_ONLINE(Verbose, TEXT("%s"), *NewEvent->ToString());
	AddToOutQueue(NewEvent);
}

#endif // WITH_TENCENT_RAIL_SDK
#endif // WITH_TENCENTSDK
