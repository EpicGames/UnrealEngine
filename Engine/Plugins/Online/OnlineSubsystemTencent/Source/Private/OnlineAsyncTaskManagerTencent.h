// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TENCENTSDK

#include "OnlineAsyncTaskManager.h"

#if WITH_TENCENT_RAIL_SDK
#include "Containers/Set.h"
#include "RailSDK.h"
#endif

class FOnlineSubsystemTencent;

/**
 *	Tencent version of the async task manager to register the various callbacks with the engine
 */

#if WITH_TENCENT_RAIL_SDK
class FOnlineAsyncTaskManagerTencent : public FOnlineAsyncTaskManager, public rail::IRailEvent
#else
class FOnlineAsyncTaskManagerTencent : public FOnlineAsyncTaskManager
#endif
{
protected:

	/** Cached reference to the main online subsystem */
	FOnlineSubsystemTencent* Subsystem;

	FOnlineAsyncTaskManagerTencent();

public:

	FOnlineAsyncTaskManagerTencent(FOnlineSubsystemTencent* InSubsystem);
	~FOnlineAsyncTaskManagerTencent();

	// FOnlineAsyncTaskManager

	/**
	 *	** CALL ONLY FROM ONLINE THREAD **
	 * Give the online service a chance to do work
	 */
	virtual void OnlineTick() override;

#if WITH_TENCENT_RAIL_SDK
	void RegisterRailEvents();
	void UnregisterRailEvents();

	void OnRailSystemStateChanged(const rail::rail_event::RailSystemStateChanged* param);
	/** Event notification that any of the floating windows have been opened or closed */
	void OnRailShowFloatingWindow(const rail::rail_event::ShowFloatingWindowResult* param);
	void OnRailShowFloatingNotifyWindow(const rail::rail_event::ShowNotifyWindow* param);
	void OnRailInviteSentNotification(const rail::rail_event::RailUsersNotifyInviter* param);
	void OnRailInviteUsersResult(const rail::rail_event::RailUsersInviteUsersResult* param);
	void OnRailInviteJoinGameResult(const rail::rail_event::RailUsersInviteJoinGameResult* param);
	void OnRailFriendsInviteRespondInvitation(const rail::rail_event::RailUsersRespondInvitation* param);
	void OnRailFriendsJoinGameByUser(const rail::rail_event::RailPlatformNotifyEventJoinGameByUser* param);
	void OnRailAssetsChanged(const rail::rail_event::RailAssetsChanged* param);
	void OnRailGetUsersInfo(const rail::rail_event::RailUsersInfoData* param);
	void OnRailGetFriendPlayedGamesResult(const rail::rail_event::RailFriendsQueryFriendPlayedGamesResult* param);
	void OnRailInviteCommandline(const rail::rail_event::RailFriendsGetInviteCommandLine* param);
	void OnRailCancelInviteResult(const rail::rail_event::RailUsersCancelInviteResult* param);
	void OnRailFriendsSetMetadataResult(const rail::rail_event::RailFriendsSetMetadataResult* param);
	void OnRailEventFriendsAddFriendResult(const rail::rail_event::RailFriendsAddFriendResult* param);
	void OnRailFriendsListChanged(const rail::rail_event::RailFriendsListChanged* param);
	/** Event notification that remote friend online state has changed */
	void OnRailFriendsOnlineStateChanged(const rail::rail_event::RailFriendsOnlineStateChanged* param);
	/** Event notification that remote friend metadata keys/values have changed */
	void OnRailFriendsMetadataChanged(const rail::rail_event::RailFriendsMetadataChanged* param);

	// Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	// End IRailEvent interface

	TSet<rail::RAIL_EVENT_ID> RegisteredRailEvents;
#endif

};

#endif // WITH_TENCENTSDK
