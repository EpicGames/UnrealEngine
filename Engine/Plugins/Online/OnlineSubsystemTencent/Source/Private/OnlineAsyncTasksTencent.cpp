// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAsyncTasksTencent.h"

#if WITH_TENCENT_RAIL_SDK

#include "Interfaces/OnlineExternalUIInterface.h"
#include "Internationalization/Culture.h"
#include "Internationalization/FastDecimalFormat.h"
#include "OnlineIdentityTencent.h"
#include "OnlineSessionTencentRail.h"
#include "OnlinePresenceTencent.h"
#include "OnlineSubsystemTencent.h"
#include "OnlineSubsystemTencentPrivate.h"
#include "MetadataKeysRail.h"
#include "OnlineSubsystemSessionSettings.h"
#include "OnlineFriendsTencent.h"
#include "OnlinePurchaseTencent.h"
#include "OnlineUserTencent.h"
#include "PlayTimeLimitImpl.h"

using namespace rail;

namespace
{
	/**
	 * Converts an engine key and its data type to an appropriate Rail key
	 * Encoded as key in the form <keyname>_<datatype> so that it is known client side
	 *
	 * @param Key online key
	 * @param Data data associated with the key
	 * @param KeyStr output Rail key for use with presence
	 *
	 * @return true if successful, false if data type unknown/unsupported
	 */
	inline bool VariantKeyToRailKey(const FString& Key, const FVariantData& Data, FString& KeyStr)
	{
		switch (Data.GetType())
		{
			case EOnlineKeyValuePairDataType::Int32:
				KeyStr = FString::Printf(TEXT("%s_i"), *Key);
				break;
			case EOnlineKeyValuePairDataType::UInt32:
				KeyStr = FString::Printf(TEXT("%s_u"), *Key);
				break;
			case EOnlineKeyValuePairDataType::Int64:
				KeyStr = FString::Printf(TEXT("%s_l"), *Key);
				break;
			case EOnlineKeyValuePairDataType::Double:
				KeyStr = FString::Printf(TEXT("%s_d"), *Key);
				break;
			case EOnlineKeyValuePairDataType::String:
				KeyStr = FString::Printf(TEXT("%s_s"), *Key);
				break;
			case EOnlineKeyValuePairDataType::Json:
				KeyStr = FString::Printf(TEXT("%s_j"), *Key);
				break;
			case EOnlineKeyValuePairDataType::Float:
				KeyStr = FString::Printf(TEXT("%s_f"), *Key);
				break;
			case EOnlineKeyValuePairDataType::Bool:
				KeyStr = FString::Printf(TEXT("%s_b"), *Key);
				break;
			case EOnlineKeyValuePairDataType::Empty:
			case EOnlineKeyValuePairDataType::Blob:
			default:
				return false;
		}

		KeyStr.ToLowerInline();
		return true;
	}

	/**
	 * Converts a Rail key back to the variant data type it was originally
	 *
	 * @param RailKey key that represents the keyname_type syntax
	 * @param RailValue the value that will be converted into a variant type
	 * @param VariantData final converted value
	 */
	inline bool RailKeyToVariantData(const ANSICHAR* RailKey, const ANSICHAR* RailValue, FString& Key, FVariantData& OutVariantData)
	{
		bool bSuccess = false;

		TCHAR RailKeyCopy[kRailCommonMaxKeyLength];

		FCString::Strncpy(RailKeyCopy, ANSI_TO_TCHAR(RailKey), UE_ARRAY_COUNT(RailKeyCopy));

		TCHAR* DataType = FCString::Strrchr(RailKeyCopy, '_');
		if (DataType)
		{
			bSuccess = true;

			// NULL Terminate the key
			*DataType = '\0';
			Key = FString(RailKeyCopy).ToLower();

			// Advance to the data type
			DataType += 1;

			switch (DataType[0])
			{
				case 'i':
					OutVariantData.SetValue((int32)0);
					OutVariantData.FromString(ANSI_TO_TCHAR(RailValue));
					break;
				case 'u':
					OutVariantData.SetValue((uint32)0);
					OutVariantData.FromString(ANSI_TO_TCHAR(RailValue));
					break;
				case 'l':
					OutVariantData.SetValue((uint64)0);
					OutVariantData.FromString(ANSI_TO_TCHAR(RailValue));
					break;
				case 'd':
					OutVariantData.SetValue((double)0);
					OutVariantData.FromString(ANSI_TO_TCHAR(RailValue));
					break;
				case 's':
					OutVariantData.SetValue(ANSI_TO_TCHAR(RailValue));
					break;
				case 'j':
					OutVariantData.SetJsonValueFromString(ANSI_TO_TCHAR(RailValue));
					break;
				case 'f':
					OutVariantData.SetValue((float)0);
					OutVariantData.FromString(ANSI_TO_TCHAR(RailValue));
					break;
				case 'b':
					OutVariantData.SetValue(false);
					OutVariantData.FromString(ANSI_TO_TCHAR(RailValue));
					break;
				default:
					bSuccess = false;
					break;
			}
		}

		if (!bSuccess)
		{
			UE_LOG_ONLINE(Warning, TEXT("Unknown or unsupported data type from Rail key data %s %s"), ANSI_TO_TCHAR(RailKey), ANSI_TO_TCHAR(RailValue));
		}

		return bSuccess;
	}

	/**
	 * Create a mapping of key/value pairs after retrieving data from Rail
	 *
	 * @param MetadataResult output from async call to Rail
	 * @param OutMetadata converted data into key/value pairs of variant data
	 */
	inline void FillMetadataFromRailResult(const rail_event::RailFriendsGetMetadataResult* MetadataResult, FMetadataPropertiesRail& OutMetadata)
	{
		for (uint32 KeyIdx = 0; KeyIdx < MetadataResult->friend_kvs.size(); ++KeyIdx)
		{
			const rail::RailKeyValueResult& KeyValuePair = MetadataResult->friend_kvs[KeyIdx];
			if (KeyValuePair.error_code == rail::RailResult::kSuccess)
			{
				FString NewKey;
				FVariantData NewValue;

				if (RailKeyToVariantData(KeyValuePair.key.c_str(), KeyValuePair.value.c_str(), NewKey, NewValue))
				{
					OutMetadata.Add(NewKey, NewValue);
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("Failed to convert metadata %s %s"), ANSI_TO_TCHAR(KeyValuePair.key.c_str()), ANSI_TO_TCHAR(KeyValuePair.value.c_str()));
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("FillMetadataFromRail Key %s returned Result: %d"), ANSI_TO_TCHAR(KeyValuePair.key.c_str()), static_cast<uint32>(KeyValuePair.error_code));
			}
		}
	}

	inline void FillMetadataFromRailResult(const rail::RailFriendMetadata& FriendMetadata, FMetadataPropertiesRail& OutMetadata)
	{
		if (FriendMetadata.friend_rail_id != rail::kInvalidRailId)
		{
			for (uint32 KeyIdx = 0; KeyIdx < FriendMetadata.metadatas.size(); ++KeyIdx)
			{
				FString NewKey;
				FVariantData NewValue;

				const RailKeyValue& KeyValuePair = FriendMetadata.metadatas[KeyIdx];
				if (RailKeyToVariantData(KeyValuePair.key.c_str(), KeyValuePair.value.c_str(), NewKey, NewValue))
				{
					OutMetadata.Add(NewKey, NewValue);
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("Failed to convert metadata %s %s"), ANSI_TO_TCHAR(KeyValuePair.key.c_str()), ANSI_TO_TCHAR(KeyValuePair.value.c_str()));
				}
			}
		}
	}

	/** @return true if the existing cache, and the data that is externally visible, needs to be updated */
	inline bool DoesMetadataNeedUpdate(const TMap<FString, FString>& MetadataCache, const FString& Key, const FString& NewValue)
	{
		const FString* ExistingValue = MetadataCache.Find(Key);
		return (!ExistingValue || (ExistingValue && !ExistingValue->Equals(NewValue, ESearchCase::CaseSensitive)));
	}

	/**
	 * Convert a raw Rail result into something resembling an FOnlineError
	 *
	 * @param InResult any rail result
	 * @param OutOnlineError the converted error result
	 */
	void ParseRailResult(rail::RailResult InResult, FOnlineError& OutOnlineError)
	{
		if (InResult != rail::kSuccess)
		{
			OutOnlineError.bSucceeded = false;
			OutOnlineError.ErrorCode = LexToString(InResult);
			OutOnlineError.ErrorRaw = FString::Printf(TEXT("0x%08x"), static_cast<int32>(InResult));
		}
		else
		{
			OutOnlineError.bSucceeded = true;
		}
	}

	/**
	 * Parse a Rail event into something resembling an FOnlineError
	 *
	 * @param InResult any rail event
	 * @param OutOnlineError the converted error result
	 */
	void ParseRailResult(const rail::EventBase* InResult, FOnlineError& OutOnlineError)
	{
		RailResult Result = InResult ? InResult->get_result() : RailResult::kErrorUnknown;
		ParseRailResult(Result, OutOnlineError);
	}

	/**
	 * Helper class to print a RailAssetInfo in JSON format
	 */
	struct FRailAssetInfoJson
		: public FJsonSerializable
	{
		FRailAssetInfoJson(rail::RailAssetInfo& InRailAssetInfo)
			: RailAssetInfo(InRailAssetInfo)
		{}

		/** The asset info to print in JSON format */
		rail::RailAssetInfo& RailAssetInfo; // non-const because our JSON serializers are used for both reading and writing and require non-const

		BEGIN_JSON_SERIALIZER
			if (Serializer.IsSaving())
			{
				static_assert(std::is_same_v<decltype(RailAssetInfo.asset_id), uint64>, "Change how we format asset_id");
				FString AssetId = FString::Printf(TEXT("%llu"), RailAssetInfo.asset_id);
				JSON_SERIALIZE("asset_id", AssetId);
				JSON_SERIALIZE("product_id", RailAssetInfo.product_id);
				FString ProductName = LexToString(RailAssetInfo.product_name);
				JSON_SERIALIZE("product_name", ProductName);
				JSON_SERIALIZE("position", RailAssetInfo.position);
				FString Progress = LexToString(RailAssetInfo.progress);
				JSON_SERIALIZE("progress", Progress);
				JSON_SERIALIZE("quantity", RailAssetInfo.quantity);
				JSON_SERIALIZE("state", RailAssetInfo.state);
				JSON_SERIALIZE("flag", RailAssetInfo.flag);
				JSON_SERIALIZE("origin", RailAssetInfo.origin);
				if (RailAssetInfo.expire_time != 0)
				{
					FDateTime ExpireTime(FDateTime::FromUnixTimestamp(static_cast<int64>(RailAssetInfo.expire_time)));
					JSON_SERIALIZE("expires", ExpireTime);
				}
				else
				{
					static FString NeverExpires(TEXT("never"));
					JSON_SERIALIZE("expires", NeverExpires);
				}
			}
		END_JSON_SERIALIZER
	};

	/**
	 * Get a JSON string representation of a RailAssetInfo
	 * @param RailAssetInfo the asset info to print in JSON format
	 * @return an FString containing the JSON string representation of the RailAssetInfo
	 */
	FString RailAssetInfoToJsonString(const rail::RailAssetInfo& RailAssetInfo)
	{
		FRailAssetInfoJson RailAssetInfoJson = {const_cast<rail::RailAssetInfo&>(RailAssetInfo)};
		return RailAssetInfoJson.ToJson(false);
	}

	void RailAssetInfosToPurchaseReceipt(const FUniqueNetId& UserId, const rail::RailArray<rail::RailAssetInfo>& AssetInfos, FPurchaseReceipt& OutPurchaseReceipt)
	{
		FPurchaseReceipt::FReceiptOfferEntry ReceiptOfferEntry(TEXT(""), TEXT(""), 1);

		for (uint32 AssetIndex = 0; AssetIndex < AssetInfos.size(); ++AssetIndex)
		{
			FPurchaseReceipt::FLineItemInfo LineItemInfo;

			const rail::RailAssetInfo& AssetInfo(AssetInfos[AssetIndex]);
			static_assert(std::is_same_v<decltype(AssetInfo.product_id), uint32>, "Change how we construct string from AssetInfo.product_id");
			static_assert(std::is_same_v<decltype(AssetInfo.asset_id), uint64>, "Change how we construct string from AssetInfo.asset_id");
			LineItemInfo.ItemName = FString::Printf(TEXT("%u"), AssetInfo.product_id);
			LineItemInfo.UniqueId = FString::Printf(TEXT("%llu"), AssetInfo.asset_id);
			if (AssetInfo.state == rail::EnumRailAssetState::kRailAssetStateNormal)
			{
				// Use the user ID as the ValidationInfo; there is no better information to provide here.
				LineItemInfo.ValidationInfo = UserId.ToString();
			}

			ReceiptOfferEntry.LineItems.Emplace(MoveTemp(LineItemInfo));
		}

		OutPurchaseReceipt.AddReceiptOffer(MoveTemp(ReceiptOfferEntry));
	}
}

void FOnlineAsyncTaskRail::Initialize()
{
	RailSdkWrapper& RailSDK = RailSdkWrapper::Get();
	if (RailSDK.IsInitialized())
	{
		for (auto EventId : RegisteredRailEvents)
		{
			RailSDK.RailRegisterEvent(EventId, this);
		}
	}
}

void FOnlineAsyncTaskRail::Tick()
{
	// Failsafe timeout to eject the task from the queue
	if (bCanTimeout)
	{
		if (GetElapsedTime() >= ASYNC_RAIL_TASK_TIMEOUT)
		{
			UE_LOG_ONLINE(Error, TEXT("TIMEOUT: %s"), *ToString());
			bIsComplete = true;
			bWasSuccessful = false;
		}
	}
}

void FOnlineAsyncTaskRail::Finalize()
{
	RailSdkWrapper& RailSDK = RailSdkWrapper::Get();
	if (RailSDK.IsInitialized())
	{
		for (auto EventId : RegisteredRailEvents)
		{
			RailSDK.RailUnregisterEvent(EventId, this);
		}
	}
}

void FOnlineAsyncTaskRail::ParseRailResult(rail::RailResult InResult, FOnlineError& OutOnlineError) const
{
	::ParseRailResult(InResult, OutOnlineError);
}

void FOnlineAsyncTaskRail::ParseRailResult(const rail::EventBase* InResult, FOnlineError& OutOnlineError) const
{
	const bool bSameAppId = (InResult && InResult->game_id.IsValid()) ? (Subsystem->GetAppId() == LexToString(InResult->game_id.get_id())) : false;
	UE_CLOG_ONLINE(InResult && !bSameAppId, Warning, TEXT("RailEvent app id mismatch %llu"), InResult->game_id.get_id());

	RailResult Result = InResult ? InResult->get_result() : RailResult::kErrorUnknown;
	::ParseRailResult(Result, OutOnlineError);
}

void FOnlineAsyncEventRail::ParseRailResult(rail::RailResult InResult, FOnlineError& OutOnlineError) const
{
	::ParseRailResult(InResult, OutOnlineError);
}

void FOnlineAsyncEventRail::ParseRailResult(const rail::EventBase* InResult, FOnlineError& OutOnlineError) const
{
	const bool bSameAppId = (InResult && InResult->game_id.IsValid()) ? (Subsystem->GetAppId() == LexToString(InResult->game_id.get_id())) : false;
	UE_CLOG_ONLINE(InResult && !bSameAppId, Warning, TEXT("RailEvent app id mismatch %llu"), InResult->game_id.get_id());

	RailResult Result = InResult ? InResult->get_result() : RailResult::kErrorUnknown;
	::ParseRailResult(Result, OutOnlineError);
}

void FOnlineAsyncTaskRailAcquireSessionTicket::Initialize()
{
	if (rail::IRailPlayer* RailPlayer = RailSdkWrapper::Get().RailPlayer())
	{
		if (RailPlayer->GetRailID() == PlayerId)
		{
			FOnlineAsyncTaskRail::Initialize();

			rail::RailString RailUserData; // @todo should we put anything here?
			RailResult Result = RailPlayer->AsyncAcquireSessionTicket(RailUserData);
			if (Result != RailResult::kSuccess)
			{
				OnlineError.SetFromErrorCode(FString::Printf(TEXT("AsyncAcquireSessionTicket initial call failed with result %u"), *LexToString(Result)));
			}
		}
		else
		{
			OnlineError.SetFromErrorCode(FString::Printf(TEXT("RailPlayer RailID does not match our expected ID (theirs %llu, ours %llu)"), RailPlayer->GetRailID().get_id(), PlayerId.get_id()));
		}
	}
	else
	{
		OnlineError.SetFromErrorCode(TEXT("RailSDK is not initialized"));
	}

	if (!OnlineError.GetErrorCode().IsEmpty())
	{
		UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskRailAcquireSessionTicket::Initialize failed %s"), *OnlineError.ToLogString());
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailAcquireSessionTicket::TriggerDelegates()
{
	FOnlineAsyncTaskRail::TriggerDelegates();
	CompletionDelegate.ExecuteIfBound(OnlineError, SessionTicket);
}

void FOnlineAsyncTaskRailAcquireSessionTicket::OnRailEvent(rail::RAIL_EVENT_ID EventId, rail::EventBase* Param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailAcquireSessionTicket::OnRailEvent"));
	switch (EventId)
	{
		case rail::kRailEventSessionTicketGetSessionTicket:
			OnRailGetSessionTicket(static_cast<rail_event::AcquireSessionTicketResponse*>(Param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailAcquireSessionTicket::OnRailGetSessionTicket(const rail::rail_event::AcquireSessionTicketResponse* SessionTicketResponse)
{
	ParseRailResult(SessionTicketResponse, OnlineError);
	UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskRailAcquireSessionTicket::OnRailGetSessionTicket SessionTicket Result: %s"), *OnlineError.ToLogString());
	
	if (OnlineError.WasSuccessful())
	{
		SessionTicket = LexToString(SessionTicketResponse->session_ticket.ticket);
		OnlineError.bSucceeded = true;
	}

	bWasSuccessful = OnlineError.WasSuccessful();
	bIsComplete = true;
}

void FOnlineAsyncTaskRailShowFloatingWindow::Initialize()
{
	RailResult Result = RailResult::kErrorUnknown;

	RailSdkWrapper& RailSDK = RailSdkWrapper::Get();
	if (RailSDK.IsInitialized())
	{
		IRailFactory* RailFactory = RailSDK.RailFactory();
		if (RailFactory)
		{
			IRailFloatingWindow* FloatingWindow = RailFactory->RailFloatingWindow();
			if (FloatingWindow)
			{
				// Register events before triggering code
				FOnlineAsyncTaskRail::Initialize();
				Result = FloatingWindow->AsyncShowRailFloatingWindow(WindowType, RailString());
				FloatingWindow->SetNotifyWindowEnable(rail::kRailNotifyWindowAntiAddiction, false); //prevent AntiAddiction message from Tencent to be shown.
			}
		}
	}

	if (Result != RailResult::kSuccess)
	{
		ParseRailResult(Result, TaskResult.Error);
		UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncShowFloatingWindow::Initialize failed %s"), *TaskResult.Error.ToLogString());
		bWasSuccessful = false;
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailShowFloatingWindow::Finalize()
{
	// Remove overload if nothing to do here

	// Unregister events last
	FOnlineAsyncTaskRail::Finalize();
}

void FOnlineAsyncTaskRailShowFloatingWindow::TriggerDelegates()
{
	// Duplicated by FOnlineAsyncEventRailShowFloatingWindow which is needed for close events
}

void FOnlineAsyncTaskRailShowFloatingWindow::OnRailEvent(RAIL_EVENT_ID event_id, EventBase* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailShowFloatingWindow::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventShowFloatingWindow:
			OnRailShowFloatingWindow(static_cast<rail_event::ShowFloatingWindowResult*>(param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailShowFloatingWindow::OnRailShowFloatingWindow(rail_event::ShowFloatingWindowResult* WindowResult)
{
	ParseRailResult(WindowResult, TaskResult.Error);
	UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncShowFloatingWindow::OnRailShowFloatingWindow Result: %s"), *TaskResult.Error.ToLogString());
	
	if (WindowResult)
	{
		TaskResult.bOpened = WindowResult->is_show;
	}

	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

bool FOnlineAsyncTaskRailSetUserMetadata::GenerateRailMetadata(FMetadataPropertiesRail& InMetadata, TMap<FString, FString>& OutFinalData, rail::RailArray<rail::RailKeyValue>& OutRailKeyValuePairs)
{
	if (InMetadata.Num() <= kRailCommonMaxRepeatedKeys)
	{
		// Compare outgoing data to existing data and remove values that are the same
		const TMap<FString, FString>& MetadataCache = Subsystem->GetMetadataCache();

		FString KeyStr;
		rail::RailKeyValue RailKey;
		for (const TPair<FString, FVariantData>& Setting : InMetadata)
		{
			if (VariantKeyToRailKey(Setting.Key, Setting.Value, KeyStr))
			{
				FString SettingStr = Setting.Value.ToString();
				UE_LOG_ONLINE(VeryVerbose, TEXT("SetMetadata [%s] %s"), *KeyStr, *SettingStr);

				if ((KeyStr.Len() <= kRailCommonMaxKeyLength) && (SettingStr.Len() <= kRailCommonMaxValueLength))
				{
					if (SettingStr.IsEmpty())
					{
						UE_LOG_ONLINE(Log, TEXT("Clearing metadata key %s"), *KeyStr);
					}

					if (DoesMetadataNeedUpdate(MetadataCache, KeyStr, SettingStr))
					{
						ToRailString(KeyStr, RailKey.key);
						ToRailString(SettingStr, RailKey.value);
						UE_LOG_ONLINE(Verbose, TEXT("SetMetadata Rail [%s] %s"), ANSI_TO_TCHAR(RailKey.key.c_str()), ANSI_TO_TCHAR(RailKey.value.c_str()));
						OutRailKeyValuePairs.push_back(RailKey);
					}
					else
					{
						UE_LOG_ONLINE(VeryVerbose, TEXT("Metadata already set, skipping key %s"), *KeyStr);
					}

					// Always record the final data, even if we aren't going to send it to RailSDK
					OutFinalData.Emplace(MoveTemp(KeyStr), MoveTemp(SettingStr));
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("Metadata too large %s %s"), *KeyStr, *Setting.Value.ToString());
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("Unsupported metadata %s %s of type %s"), *Setting.Key, *Setting.Value.ToString(), EOnlineKeyValuePairDataType::ToString(Setting.Value.GetType()));
			}
		}

		return true;
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("Too many keys to set metadata %d"), InMetadata.Num());
		return false;
	}
}

void FOnlineAsyncTaskRailSetUserMetadata::Initialize()
{
	UE_LOG_ONLINE(Verbose, TEXT("FOnlineAsyncTaskRailSetUserMetadata::Initialize"));

	rail::RailArray<rail::RailKeyValue> RailKeyValuePairs;
	rail::RailResult RailResult = rail::RailResult::kErrorUnknown;

	if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
	{
		if (GenerateRailMetadata(Metadata, TaskResult.FinalData, RailKeyValuePairs))
		{
			RailResult = rail::RailResult::kSuccess;
			if (RailKeyValuePairs.size() > 0)
			{
				FOnlineAsyncTaskRail::Initialize();
				UE_LOG_ONLINE(VeryVerbose, TEXT("- setting %d keys"), RailKeyValuePairs.size());
				RailResult = RailFriends->AsyncSetMyMetadata(RailKeyValuePairs, RailString());
			}
		}
	}

	if ((RailResult != rail::RailResult::kSuccess) || (RailKeyValuePairs.size() == 0))
	{
		ParseRailResult(RailResult, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailSetUserMetadata::OnRailEvent(RAIL_EVENT_ID event_id, EventBase* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailSetUserMetadata::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventFriendsSetMetadataResult:
			OnRailEventFriendsSetMetadataResult(static_cast<rail_event::RailFriendsSetMetadataResult*>(param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailSetUserMetadata::OnRailEventFriendsSetMetadataResult(const rail_event::RailFriendsSetMetadataResult* MetadataResult)
{
	ParseRailResult(MetadataResult, TaskResult.Error);
	UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskRailSetUserMetadata::OnRailEventFriendsSetMetadataResult Result: %s"), *TaskResult.Error.ToLogString());

	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

void FOnlineAsyncTaskRailSetUserMetadata::Finalize()
{
	// Add all keys set here for debugging purposes
	Subsystem->AddToMetadataCache(TaskResult.FinalData);

	// Unregister events last
	FOnlineAsyncTaskRail::Finalize();
}

void FOnlineAsyncTaskRailSetUserMetadata::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

bool FOnlineAsyncTaskRailSetUserPresence::GenerateRailMetadata(FMetadataPropertiesRail& InMetadata, TMap<FString, FString>& OutFinalData, rail::RailArray<rail::RailKeyValue>& OutRailKeyValuePairs)
{
	bool bSuccess = false;
	if ((InMetadata.Num() + 1) <= kRailCommonMaxRepeatedKeys)
	{
		// Generate the expected keys
		if (FOnlineAsyncTaskRailSetUserMetadata::GenerateRailMetadata(InMetadata, OutFinalData, OutRailKeyValuePairs))
		{
			// Mark all the presence keys set above for later retrieval
			FString KeyStr = RAIL_PRESENCE_PRESENCE_KEYS TEXT("_s");

			TArray<FString> FinalKeys;
			OutFinalData.GenerateKeyArray(FinalKeys);
			FString ValueStr = FString::Join(FinalKeys, RAIL_METADATA_KEY_SEPARATOR);
			if ((KeyStr.Len() <= kRailCommonMaxKeyLength) && (ValueStr.Len() <= kRailCommonMaxValueLength))
			{
				const TMap<FString, FString>& MetadataCache = Subsystem->GetMetadataCache();
				if (DoesMetadataNeedUpdate(MetadataCache, KeyStr, ValueStr))
				{
					rail::RailKeyValue RailKey;
					ToRailString(KeyStr, RailKey.key);
					ToRailString(ValueStr, RailKey.value);
					OutRailKeyValuePairs.push_back(RailKey);
				}
				else
				{
					UE_LOG_ONLINE(VeryVerbose, TEXT("Metadata already set, skipping key %s"), *KeyStr);
				}

				// Always record the final data, even if we aren't going to send it to RailSDK
				OutFinalData.Emplace(MoveTemp(KeyStr), MoveTemp(ValueStr));
				bSuccess = true;
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("Presence metadata too large %s %s"), *KeyStr, *ValueStr);
			}
		}
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("Too many keys to set presence %d"), InMetadata.Num());
	}

	return bSuccess;
}

void FOnlineAsyncTaskRailSetSessionMetadata::Initialize()
{
	UE_LOG_ONLINE(Verbose, TEXT("FOnlineAsyncTaskRailSetSessionMetadata::Initialize"));

	rail::RailArray<rail::RailKeyValue> RailKeyValuePairs;
	rail::RailResult RailResult = rail::RailResult::kErrorUnknown;

	IOnlineSessionPtr SessionInt = Subsystem->GetSessionInterface();
	if (SessionInt.IsValid())
	{
		FOnlineSessionTencentRailPtr TencentSessionInt = StaticCastSharedPtr<FOnlineSessionTencentRail>(SessionInt);
		if (TencentSessionInt.IsValid())
		{
			if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
			{
				if (GenerateRailMetadata(Metadata, TaskResult.FinalData, RailKeyValuePairs))
				{
					// Clear any keys previously set by session that is no longer referenced
					// Set here so that any previous task executing while this is pending can be included
					const TArray<FString>& CurrentKeys = TencentSessionInt->GetCurrentPresenceKeys();
					for (const FString& Key : CurrentKeys)
					{
						if (!TaskResult.FinalData.Contains(Key))
						{
							rail::RailKeyValue RailKey;
							ToRailString(Key, RailKey.key);
							RailKey.value = rail::RailString();
							UE_LOG_ONLINE(Verbose, TEXT("Clearing previous key %s"), ANSI_TO_TCHAR(RailKey.key.c_str()));
							RailKeyValuePairs.push_back(RailKey);
							TaskResult.FinalData.Add(Key, FString());
						}
					}

					RailResult = rail::RailResult::kSuccess;
					if (RailKeyValuePairs.size() > 0)
					{
						FOnlineAsyncTaskRail::Initialize();
						UE_LOG_ONLINE(VeryVerbose, TEXT("- setting %d keys"), RailKeyValuePairs.size());
						RailResult = RailFriends->AsyncSetMyMetadata(RailKeyValuePairs, RailString());
					}
				}
			}
		}
	}

	if ((RailResult != rail::RailResult::kSuccess) || (RailKeyValuePairs.size() == 0))
	{
		ParseRailResult(RailResult, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailSetSessionMetadata::Finalize()
{
	FOnlineSessionTencentRailPtr SessionInt = StaticCastSharedPtr<FOnlineSessionTencentRail>(Subsystem->GetSessionInterface());
	if (SessionInt.IsValid())
	{
		// Mark the newly active session keys
		SessionInt->CurrentSessionPresenceKeys.Empty(TaskResult.FinalData.Num());
		if (TaskResult.FinalData.Num() > 0)
		{
			TaskResult.FinalData.GenerateKeyArray(SessionInt->CurrentSessionPresenceKeys);
		}
	}

	FOnlineAsyncTaskRailSetUserMetadata::Finalize();
}

void FOnlineAsyncTaskRailSetInviteCommandline::Initialize()
{
	UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskRailSetInviteCommandline::Initialize"));
	rail::RailResult RailResult = rail::RailResult::kErrorUnknown;

	if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
	{
		UE_LOG_ONLINE(Verbose, TEXT("Invite command line: %s"), *Cmdline);

		rail::RailString RailCmdLine;
		ToRailString(Cmdline, RailCmdLine);

		FOnlineAsyncTaskRail::Initialize();
		RailResult = RailFriends->AsyncSetInviteCommandLine(RailCmdLine, RailString());
	}

	if (RailResult != rail::RailResult::kSuccess)
	{
		ParseRailResult(RailResult, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailSetInviteCommandline::OnRailEvent(RAIL_EVENT_ID event_id, EventBase* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailSetInviteCommandline::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventFriendsSetMetadataResult:
			OnRailEventFriendsSetMetadataResult(static_cast<rail_event::RailFriendsSetMetadataResult*>(param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailSetInviteCommandline::OnRailEventFriendsSetMetadataResult(const rail_event::RailFriendsSetMetadataResult* MetadataResult)
{
	ParseRailResult(MetadataResult, TaskResult.Error);
	UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskRailSetInviteCommandline::OnRailEventFriendsSetMetadataResult Result: %s"), *TaskResult.Error.ToLogString());

	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

void FOnlineAsyncTaskRailSetInviteCommandline::Finalize()
{
	// Remove overload if nothing to do here

	// Unregister events last
	FOnlineAsyncTaskRail::Finalize();
}

void FOnlineAsyncTaskRailSetInviteCommandline::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

FOnlineAsyncTaskRailGetUserMetadata::FOnlineAsyncTaskRailGetUserMetadata(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const TArray<FString>& InMetadataKeys, const FOnOnlineAsyncTaskRailGetUserMetadataComplete& InCompletionDelegate)
	: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventFriendsGetMetadataResult})
	, RailUserId((rail::RailID)InUserId)
	, MetadataKeys(InMetadataKeys)
	, CompletionDelegate(InCompletionDelegate)
{
}

FOnlineAsyncTaskRailGetUserMetadata::~FOnlineAsyncTaskRailGetUserMetadata()
{
}

void FOnlineAsyncTaskRailGetUserMetadata::Initialize()
{
	TaskResult.UserId = FUniqueNetIdRail::Create(RailUserId);

	rail::RailResult Result = QueryMetadata(MetadataKeys);
	if ((Result != rail::RailResult::kSuccess) || (MetadataKeys.Num() == 0))
	{
		ParseRailResult(Result, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

rail::RailResult FOnlineAsyncTaskRailGetUserMetadata::QueryMetadata(const TArray<FString>& InMetadata)
{
	rail::RailResult Result = rail::RailResult::kErrorUnknown;
	if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
	{
		Result = rail::RailResult::kSuccess;
		if (InMetadata.Num() > 0)
		{
			rail::RailArray<rail::RailString> RailPresenceKeys(InMetadata.Num());
			for (int32 KeyIdx = 0; KeyIdx < InMetadata.Num(); ++KeyIdx)
			{
				const FPresenceKey& KeyStr = InMetadata[KeyIdx];
				rail::RailString Key;
				ToRailString(KeyStr, Key);
				RailPresenceKeys.push_back(Key);
			}
			ensure(RailPresenceKeys.size() == InMetadata.Num());
			FOnlineAsyncTaskRail::Initialize();
			Result = RailFriends->AsyncGetFriendMetadata(RailUserId, RailPresenceKeys, RailString());
		}
	}
	
	return Result;
}

void FOnlineAsyncTaskRailGetUserMetadata::OnRailEvent(RAIL_EVENT_ID event_id, EventBase* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailGetUserMetadata::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventFriendsGetMetadataResult:
			OnRailEventFriendsGetMetadataResult(static_cast<rail_event::RailFriendsGetMetadataResult*>(param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailGetUserMetadata::OnRailEventFriendsGetMetadataResult(const rail_event::RailFriendsGetMetadataResult* MetadataResult)
{
	ParseRailResult(MetadataResult, TaskResult.Error);
	UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskRailGetUserMetadata::OnRailEventFriendsGetMetadataResult Result: %s"), *TaskResult.Error.ToLogString());

	if (MetadataResult)
	{
		ensure((rail::RailID)(*TaskResult.UserId) == MetadataResult->friend_id);
		FillMetadataFromRailResult(MetadataResult, TaskResult.Metadata);
	}

	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

void FOnlineAsyncTaskRailGetUserMetadata::Finalize()
{
	// Remove overload if nothing to do here

	// Unregister events last
	FOnlineAsyncTaskRail::Finalize();
}

void FOnlineAsyncTaskRailGetUserMetadata::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

FString FOnlineAsyncTaskRailGetUserMetadata::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskRailGetUserMetadata %s bWasSuccessful: %d"), TaskResult.UserId.IsValid() ? *TaskResult.UserId->ToString() : TEXT("Invalid"), WasSuccessful());
}

FOnlineAsyncTaskRailGetUserPresence::FOnlineAsyncTaskRailGetUserPresence(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const FOnOnlineAsyncTaskRailGetUserMetadataComplete& InCompletionDelegate)
	: FOnlineAsyncTaskRailGetUserMetadata(InSubsystem, InUserId, TArray<FString>(), InCompletionDelegate)
	, bKeysQueried(false)
{

}

FOnlineAsyncTaskRailGetUserPresence::~FOnlineAsyncTaskRailGetUserPresence()
{
}

void FOnlineAsyncTaskRailGetUserPresence::Initialize() 
{
	TaskResult.UserId = FUniqueNetIdRail::Create(RailUserId);

	TArray<FString> PresenceKey;
	PresenceKey.Add(RAIL_PRESENCE_PRESENCE_KEYS TEXT("_s"));
	rail::RailResult Result = QueryMetadata(PresenceKey);
	if (Result != rail::RailResult::kSuccess)
	{
		ParseRailResult(Result, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

FString FOnlineAsyncTaskRailGetUserPresence::ToString() const
{
	return FString::Printf(TEXT("FOnlineAsyncTaskRailGetUserPresence %s bWasSuccessful: %d"), TaskResult.UserId.IsValid() ? *TaskResult.UserId->ToString() : TEXT("Invalid"), WasSuccessful());
}

void FOnlineAsyncTaskRailGetUserPresence::OnRailEventFriendsGetMetadataResult(const rail::rail_event::RailFriendsGetMetadataResult* MetadataResult)
{
	if (bKeysQueried)
	{
		// Retrieve the real data
		FOnlineAsyncTaskRailGetUserMetadata::OnRailEventFriendsGetMetadataResult(MetadataResult);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
	else
	{
		bKeysQueried = true;

		// Retrieve the keys
		FMetadataPropertiesRail PresenceKey;
		FillMetadataFromRailResult(MetadataResult, PresenceKey);
		if (PresenceKey.Num() > 0)
		{
			FString PresenceKeysStr;
			FVariantData* PresenceData = PresenceKey.Find(RAIL_PRESENCE_PRESENCE_KEYS);
			if (PresenceData)
			{
				PresenceData->GetValue(PresenceKeysStr);
				if (!PresenceKeysStr.IsEmpty())
				{
					TArray<FString> PresenceKeys;
					int32 NumKeys = PresenceKeysStr.ParseIntoArray(PresenceKeys, RAIL_METADATA_KEY_SEPARATOR);
					rail::RailResult Result = QueryMetadata(PresenceKeys);
					if ((Result != rail::RailResult::kSuccess) || (PresenceKeys.Num() == 0))
					{
						ParseRailResult(Result, TaskResult.Error);
						bWasSuccessful = TaskResult.Error.WasSuccessful();
						bIsComplete = true;
					}
					else
					{
						// Mark successful so far
						TaskResult.Error.bSucceeded = true;
					}
				}
				else
				{
					// User not online or presence is empty for some reason
					TaskResult.Error.bSucceeded = true;
					bWasSuccessful = true;
					bIsComplete = true;
				}
			}
			else
			{
				// Query failure - somehow had the key but something went wrong
				UE_LOG_ONLINE_PRESENCE(Warning, TEXT("FOnlineAsyncTaskRailGetUserPresence failure"));
				TaskResult.Error.SetFromErrorCode(TEXT("railPresence.query_failure"));
			}
		}
		else
		{
			// User not online or not in our engine with our keys
			TaskResult.Error.bSucceeded = true;
			bWasSuccessful = true;
			bIsComplete = true;
		}

		if (!TaskResult.Error.WasSuccessful())
		{
			bWasSuccessful = false;
			bIsComplete = true;
		}
	}
}

FOnlineAsyncTaskRailGetInviteCommandline::FOnlineAsyncTaskRailGetInviteCommandline(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const FOnOnlineAsyncTaskRailGetInviteCommandLineComplete& InCompletionDelegate)
	: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventFriendsGetInviteCommandLine})
	, RailUserId((rail::RailID)InUserId)
	, CompletionDelegate(InCompletionDelegate)
{
}

FOnlineAsyncTaskRailGetInviteCommandline::~FOnlineAsyncTaskRailGetInviteCommandline()
{
}

void FOnlineAsyncTaskRailGetInviteCommandline::Initialize()
{
	rail::RailResult Result = rail::RailResult::kErrorUnknown;
	TaskResult.UserId = FUniqueNetIdRail::Create(RailUserId);

	if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
	{
		FOnlineAsyncTaskRail::Initialize();
		Result = RailFriends->AsyncGetInviteCommandLine(RailUserId, RailString());
	}

	if (Result != rail::RailResult::kSuccess)
	{
		ParseRailResult(Result, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailGetInviteCommandline::OnRailEvent(RAIL_EVENT_ID event_id, EventBase* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailGetInviteCommandline::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventFriendsGetInviteCommandLine:
			OnRailEventFriendsGetInviteCommandLine(static_cast<rail_event::RailFriendsGetInviteCommandLine*>(param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailGetInviteCommandline::OnRailEventFriendsGetInviteCommandLine(const rail_event::RailFriendsGetInviteCommandLine* InviteResult)
{
	ParseRailResult(InviteResult, TaskResult.Error);
	UE_LOG_ONLINE(Verbose, TEXT("FOnlineAsyncTaskRailGetInviteCommandline::OnRailEventFriendsGetInviteCommandLine Result: %s"), *TaskResult.Error.ToLogString());

	if (InviteResult)
	{
		ensure((rail::RailID)(*TaskResult.UserId) == InviteResult->friend_id);
		TaskResult.Commandline = FString(ANSI_TO_TCHAR(InviteResult->invite_command_line.c_str()));
	}

	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

void FOnlineAsyncTaskRailGetInviteCommandline::Finalize()
{
	// Remove overload if nothing to do here

	// Unregister events last
	FOnlineAsyncTaskRail::Finalize();
}

void FOnlineAsyncTaskRailGetInviteCommandline::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

FOnlineAsyncTaskRailGetUserInvite::FOnlineAsyncTaskRailGetUserInvite(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const FOnOnlineAsyncTaskRailGetUserInviteComplete& InCompletionDelegate)
	: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventFriendsGetMetadataResult, rail::kRailEventFriendsGetInviteCommandLine})
	, bMetadataReceived(false)
	, MetadataReceivedResult(rail::RailResult::kErrorUnknown)
	, bCommandLineReceived(false)
	, CommandLineResult(rail::RailResult::kErrorUnknown)
	, RailUserId((rail::RailID)InUserId)
	, CompletionDelegate(InCompletionDelegate)
{
}

FOnlineAsyncTaskRailGetUserInvite::~FOnlineAsyncTaskRailGetUserInvite()
{
}

void FOnlineAsyncTaskRailGetUserInvite::Initialize()
{
	rail::RailResult Result = rail::RailResult::kErrorUnknown;
	TaskResult.UserId = FUniqueNetIdRail::Create(RailUserId);

	if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
	{
		FOnlineAsyncTaskRail::Initialize();

		// Retrieve the command line so that we can get the metadata keys from the remote user related to the invite
		Result = RailFriends->AsyncGetInviteCommandLine(RailUserId, RailString());
		if (Result != RailResult::kSuccess)
		{
			bCommandLineReceived = true;
			CommandLineResult = Result;
			bMetadataReceived = true;
		}
	}
	else
	{
		bCommandLineReceived = true;
		bMetadataReceived = true;
	}

	// If either call is in flight, let it finish before finalizing
	if (bCommandLineReceived && bMetadataReceived)
	{
		OnEventComplete();
	}
}

void FOnlineAsyncTaskRailGetUserInvite::OnRailEvent(RAIL_EVENT_ID event_id, EventBase* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailGetUserInvite::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventFriendsGetMetadataResult:
			OnRailEventFriendsGetMetadataResult(static_cast<rail_event::RailFriendsGetMetadataResult*>(param));
			break;
		case rail::kRailEventFriendsGetInviteCommandLine:
			OnRailEventFriendsGetInviteCommandLine(static_cast<rail_event::RailFriendsGetInviteCommandLine*>(param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailGetUserInvite::OnRailEventFriendsGetMetadataResult(const rail_event::RailFriendsGetMetadataResult* MetadataResult)
{
	MetadataReceivedResult = MetadataResult ? MetadataResult->get_result() : RailResult::kErrorUnknown;
	UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskRailGetUserInvite::OnRailEventFriendsGetMetadataResult Result: %s"), *LexToString(MetadataReceivedResult));

	if (MetadataResult)
	{
		ensure((rail::RailID)(*TaskResult.UserId) == MetadataResult->friend_id);
		FillMetadataFromRailResult(MetadataResult, TaskResult.Metadata);
	}

	bMetadataReceived = true;
	OnEventComplete();
}

void FOnlineAsyncTaskRailGetUserInvite::OnRailEventFriendsGetInviteCommandLine(const rail_event::RailFriendsGetInviteCommandLine* InviteResult)
{
	CommandLineResult = InviteResult ? InviteResult->get_result() : RailResult::kErrorUnknown;
	UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskRailGetUserInvite::OnRailEventFriendsGetInviteCommandLine Result: %s"), *LexToString(CommandLineResult));

	if (InviteResult)
	{
		ensure((rail::RailID)(*TaskResult.UserId) == InviteResult->friend_id);
		TaskResult.Commandline = FString(ANSI_TO_TCHAR(InviteResult->invite_command_line.c_str()));
	}

	bCommandLineReceived = true;
	// Possibly kick off the async key retrieval
	RetrieveFromInviteCommandLine(TaskResult.Commandline);
	// Event will end if there was nothing to do above
	OnEventComplete();
}

void FOnlineAsyncTaskRailGetUserInvite::RetrieveFromInviteCommandLine(const FString& InviteCommandline)
{
	if (!InviteCommandline.IsEmpty())
	{
		InviteCommandline.ParseIntoArray(MetadataKeys, RAIL_METADATA_KEY_SEPARATOR);
	}

	if (MetadataKeys.Num() > 0)
	{
		rail::RailArray<rail::RailString> RailPresenceKeys(MetadataKeys.Num());
		for (int32 KeyIdx = 0; KeyIdx < MetadataKeys.Num(); ++KeyIdx)
		{
			const FPresenceKey& KeyStr = MetadataKeys[KeyIdx];
			rail::RailString Key;
			ToRailString(KeyStr, Key);
			RailPresenceKeys.push_back(Key);
		}
		ensure(RailPresenceKeys.size() == MetadataKeys.Num());

		if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
		{
			rail::RailResult Result = RailFriends->AsyncGetFriendMetadata(RailUserId, RailPresenceKeys, RailString());
			if (Result != RailResult::kSuccess)
			{
				bMetadataReceived = true;
				MetadataReceivedResult = Result;
			}
		}
		else
		{
			bMetadataReceived = true;
			MetadataReceivedResult = RailResult::kErrorFriends;
		}
	}
	else
	{
		// No keys to retrieve, mark this as success
		bMetadataReceived = true;
		MetadataReceivedResult = RailResult::kSuccess;
	}
}

void FOnlineAsyncTaskRailGetUserInvite::OnEventComplete()
{
	if (bCommandLineReceived && bMetadataReceived)
	{
		if ((MetadataReceivedResult == RailResult::kSuccess) && (CommandLineResult == RailResult::kSuccess))
		{
			TaskResult.Error.bSucceeded = true;
		}
		else if ((MetadataReceivedResult != RailResult::kSuccess) && (CommandLineResult != RailResult::kSuccess))
		{
			TaskResult.Error.SetFromErrorCode(FString::Printf(TEXT("Both async tasks failed Meta: %d CmdLine: %d"), *LexToString(MetadataReceivedResult), *LexToString(CommandLineResult)));
		}
		else if (MetadataReceivedResult != RailResult::kSuccess)
		{
			ParseRailResult(MetadataReceivedResult, TaskResult.Error);
		}
		else if (CommandLineResult != RailResult::kSuccess)
		{
			ParseRailResult(CommandLineResult, TaskResult.Error);
		}

		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailGetUserInvite::Finalize()
{
	// Remove overload if nothing to do here

	// Unregister events last
	FOnlineAsyncTaskRail::Finalize();
}

void FOnlineAsyncTaskRailGetUserInvite::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

FOnlineAsyncTaskRailClearAllMetadata::FOnlineAsyncTaskRailClearAllMetadata(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId)
	: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventFriendsClearMetadataResult})
	, RailUserId((rail::RailID)InUserId)
{
}

FOnlineAsyncTaskRailClearAllMetadata::~FOnlineAsyncTaskRailClearAllMetadata()
{
}

void FOnlineAsyncTaskRailClearAllMetadata::Initialize()
{
	rail::RailResult Result = rail::RailResult::kErrorUnknown;
	if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
	{
		FOnlineAsyncTaskRail::Initialize();
		Result = RailFriends->AsyncClearAllMyMetadata(RailString());
	}
	
	if (Result != rail::RailResult::kSuccess)
	{
		ParseRailResult(Result, TaskResult);
		bWasSuccessful = TaskResult.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailClearAllMetadata::OnRailEvent(RAIL_EVENT_ID event_id, EventBase* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailClearAllMetadata::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventFriendsClearMetadataResult:
			OnRailEventFriendsClearAllMetadataResult(static_cast<rail_event::RailFriendsClearMetadataResult*>(param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailClearAllMetadata::OnRailEventFriendsClearAllMetadataResult(const rail_event::RailFriendsClearMetadataResult* MetadataResult)
{
	ParseRailResult(MetadataResult, TaskResult);
	UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskRailClearAllMetadata::OnRailEventFriendsClearAllMetadataResult Result: %s"), *TaskResult.ToLogString());

	bWasSuccessful = TaskResult.WasSuccessful();
	bIsComplete = true;
}

void FOnlineAsyncTaskRailClearAllMetadata::Finalize()
{
	// Remove overload if nothing to do here

	// Unregister events last
	FOnlineAsyncTaskRail::Finalize();
}

void FOnlineAsyncTaskRailClearAllMetadata::TriggerDelegates()
{
}

FOnlineAsyncTaskRailAddFriend::FOnlineAsyncTaskRailAddFriend(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const FOnOnlineAsyncTaskRailAddFriendComplete& InCompletionDelegate)
	: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventFriendsAddFriendResult})
	, RailUserId((rail::RailID)InUserId)
	, CompletionDelegate(InCompletionDelegate)
{
}

FOnlineAsyncTaskRailAddFriend::~FOnlineAsyncTaskRailAddFriend()
{
}

void FOnlineAsyncTaskRailAddFriend::Initialize()
{
	rail::RailResult Result = rail::RailResult::kErrorUnknown;
	if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
	{
		FOnlineAsyncTaskRail::Initialize();

		RailFriendsAddFriendRequest AddFriendRequest;
		AddFriendRequest.target_rail_id = RailUserId;
		
		Result = RailFriends->AsyncAddFriend(AddFriendRequest, rail::RailString());
	}
	
	if (Result != rail::RailResult::kSuccess)
	{
		ParseRailResult(Result, TaskResult);
		bWasSuccessful = TaskResult.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailAddFriend::OnRailEvent(RAIL_EVENT_ID event_id, EventBase* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailClearAllMetadata::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventFriendsAddFriendResult:
		{
			OnRailEventFriendsAddFriendResult(static_cast<rail_event::RailFriendsAddFriendResult*>(param));
			break;
		}
		default:
			break;
	}
}

void FOnlineAsyncTaskRailAddFriend::OnRailEventFriendsAddFriendResult(const rail_event::RailFriendsAddFriendResult* AddFriendResult)
{
	ParseRailResult(AddFriendResult, TaskResult);
	UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncTaskRailAddFriend::OnRailEventFriendsAddFriendResult Result: %s"), *TaskResult.ToLogString());

	bWasSuccessful = TaskResult.WasSuccessful();
	bIsComplete = true;
}

void FOnlineAsyncTaskRailAddFriend::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

FThreadSafeCounter FOnlineAsyncTaskRailGetUsersInfo::RequestIdCounter;

FOnlineAsyncTaskRailGetUsersInfo::FOnlineAsyncTaskRailGetUsersInfo(FOnlineSubsystemTencent* InSubsystem, const TArray<FUniqueNetIdRef>& InUserIds, const FOnOnlineAsyncTaskRailGetUsersInfoComplete& InCompletionDelegate)
	: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventUsersGetUsersInfo})
	, UserIds(InUserIds)
	, CompletionDelegate(InCompletionDelegate)
{
	RequestId = RequestIdCounter.Increment();
}

void FOnlineAsyncTaskRailGetUsersInfo::Initialize()
{
	// TODO:  Break into batches of kRailCommonMaxRepeatedKeys
	rail::RailResult Result = rail::RailResult::kErrorUnknown;
	if (rail::IRailUsersHelper* const UsersHelper = RailSdkWrapper::Get().RailUsersHelper())
	{
		FOnlineAsyncTaskRail::Initialize();
		rail::RailArray<rail::RailID> RailIds;
		for (const FUniqueNetIdRef& UserId : UserIds)
		{
			RailIds.push_back(StaticCastSharedRef<const FUniqueNetIdRail>(UserId)->RailID);
		}
		const FString UserData = FString::FromInt(RequestId);
		rail::RailString RailUserData;
		ToRailString(UserData, RailUserData);
		Result = UsersHelper->AsyncGetUsersInfo(RailIds, RailUserData);
	}
	
	if (Result != rail::RailResult::kSuccess)
	{
		ParseRailResult(Result, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailGetUsersInfo::OnRailEvent(RAIL_EVENT_ID event_id, EventBase* param)
{
	const int32 ParamRequestId = FCStringAnsi::Atoi(param->user_data.c_str());
	const bool bMatchesRequestId = (ParamRequestId == RequestId);
	if (bMatchesRequestId)
	{
		UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailGetUsersInfo::OnRailEvent"));
		switch (event_id)
		{
			case rail::kRailEventUsersGetUsersInfo:
				OnRailEventUsersGetUsersInfo(static_cast<rail_event::RailUsersInfoData*>(param));
				break;
			default:
				break;
		}
	}
	else
	{
		const bool bParamRequestIdIsEmpty = (param->user_data.size() == 0);
		if (bParamRequestIdIsEmpty)
		{
			UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskRailGetUsersInfo::OnRailEvent: Received event %d with empty user_data"), static_cast<int32>(event_id));
		}
	}
}

void FOnlineAsyncTaskRailGetUsersInfo::OnRailEventUsersGetUsersInfo(const rail_event::RailUsersInfoData* UsersInfoData)
{
	ParseRailResult(UsersInfoData, TaskResult.Error);
	UE_LOG_ONLINE(Log, TEXT("OnRailEventUsersGetUsersInfo Result: %s"), *TaskResult.Error.ToLogString());

	if (UsersInfoData)
	{
		TaskResult.UserInfos.Reserve(UsersInfoData->user_info_list.size());
		for (uint32 Index = 0; Index < UsersInfoData->user_info_list.size(); ++Index)
		{
			const rail::PlayerPersonalInfo& RailInfo = UsersInfoData->user_info_list[Index];
			if (RailInfo.error_code == rail::kSuccess)
			{
				FUniqueNetIdRailRef RailId = FUniqueNetIdRail::Create(RailInfo.rail_id);
				FOnlineUserInfoTencentRef OnlineUser = MakeShared<FOnlineUserInfoTencent>(RailId);
				OnlineUser->SetUserAttribute(USER_ATTR_DISPLAYNAME, LexToString(RailInfo.rail_name));
				TaskResult.UserInfos.Emplace(OnlineUser);
			}
			else
			{
				UE_LOG_ONLINE(Log, TEXT("OnRailEventUsersGetUsersInfo: User %llu has error %s"), RailInfo.rail_id.get_id(), *LexToString(RailInfo.error_code));
			}
		}
	}

	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

void FOnlineAsyncTaskRailGetUsersInfo::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

void FOnlineAsyncEventRailSystemStateChanged::Finalize()
{

}

void FOnlineAsyncEventRailSystemStateChanged::TriggerDelegates()
{
	switch (State)
	{
	case rail::RailSystemState::kSystemStateUnknown:
	case rail::RailSystemState::kSystemStatePlatformOffline:
	case rail::RailSystemState::kSystemStatePlatformExit:
		UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncEventRailSystemStateChanged: Calling identity logout on system state %s"), *LexToString(State));
		{
			IOnlineIdentityPtr Identity = Subsystem->GetIdentityInterface();
			if (ensure(Identity.IsValid()))
			{
				static const constexpr int32 LocalUserNum = 0;
				Identity->Logout(LocalUserNum);
			}
		}
		break;
	case rail::RailSystemState::kSystemStatePlatformOnline:
		UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncEventRailSystemStateChanged: Platform went online, but not doing anything, waiting for game to call identity login"));
		break;

	case rail::RailSystemState::kSystemStatePlayerOwnershipExpired:
	case rail::RailSystemState::kSystemStatePlayerOwnershipActivated:
		UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncEventRailSystemStateChanged: Unsure of what to do for player ownership state %s"), *LexToString(State));
		break;
	case rail::RailSystemState::kSystemStateGameExitByAntiAddiction:
		UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncEventRailSystemStateChanged: Exiting the game by AntiAddiction on state %s"), *LexToString(State));
		FPlayTimeLimitImpl::Get().GameExitByRequest();
		break;

	default:
		UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncEventRailSystemStateChanged: Unexpected state %s"), *LexToString(State));
		break;
	}
}

FOnlineAsyncEventRailShowFloatingWindow::FOnlineAsyncEventRailShowFloatingWindow(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::ShowFloatingWindowResult* InParams)
	: FOnlineAsyncEventRail(InSubsystem)
{
	ParseRailResult(InParams, TaskResult.Error);
	TaskResult.bOpened = InParams ? InParams->is_show : false;
}

void FOnlineAsyncEventRailShowFloatingWindow::Finalize()
{
	// Remove overload if nothing to do here
	FOnlineAsyncEventRail::Finalize();
}

void FOnlineAsyncEventRailShowFloatingWindow::TriggerDelegates()
{
	FOnlineAsyncEventRail::TriggerDelegates();
	IOnlineExternalUIPtr ExternalUIInterface = Subsystem->GetExternalUIInterface();
	ExternalUIInterface->TriggerOnExternalUIChangeDelegates(TaskResult.bOpened);
}

FOnlineAsyncEventRailShowFloatingNotifyWindow::FOnlineAsyncEventRailShowFloatingNotifyWindow(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::ShowNotifyWindow* InParams)
	: FOnlineAsyncEventRail(InSubsystem)
	, bShowAntiAddictionMessage(false)
{	
	if (InParams && InParams->window_type == rail::kRailNotifyWindowAntiAddiction)
	{
		// Currently only sending Event notification for Anti-Addiction messages
		bShowAntiAddictionMessage = true;
		Payload.FromJson(LexToString(InParams->json_content));
	}
}

void FOnlineAsyncEventRailShowFloatingNotifyWindow::TriggerDelegates()
{
	if (bShowAntiAddictionMessage)
	{
		FOnlineSubsystemTencent* OSS = StaticCast<FOnlineSubsystemTencent*>(IOnlineSubsystem::Get(TENCENT_SUBSYSTEM));
		if (OSS)
		{
			OSS->TriggerOnAASDialogDelegates(Payload.DialogTitle, Payload.DialogText, Payload.ButtonText);
		}
	}
}

void FOnlineAsyncEventRailInviteSent::Finalize()
{
	// nothing to do locally now that the invite has been sent
}

void FOnlineAsyncEventRailInviteSent::TriggerDelegates()
{
	// nothing to do locally now that the invite has been sent
}

void FOnlineAsyncEventRailInviteSentEx::Finalize()
{
	// nothing to do locally now that the invite has been sent
}

void FOnlineAsyncEventRailInviteSentEx::TriggerDelegates()
{
	// nothing to do locally now that the invite has been sent
}

void FOnlineAsyncEventRailInviteResponse::Finalize()
{
	if (Response == rail::kRailInviteResponseTypeAccepted)
	{
		IOnlineIdentityPtr IdentityInt = Subsystem->GetIdentityInterface();
		if (IdentityInt.IsValid())
		{
			FUniqueNetIdRailRef UserId = FUniqueNetIdRail::Create(InviteeId);
			if (IdentityInt->GetLoginStatus(*UserId) == ELoginStatus::LoggedIn)
			{
				FUniqueNetIdRailRef RemoteUserId = FUniqueNetIdRail::Create(InviterId);

				FOnlineSessionTencentRailPtr SessionInt = StaticCastSharedPtr<FOnlineSessionTencentRail>(Subsystem->GetSessionInterface());
				if (SessionInt.IsValid())
				{
					SessionInt->QueryAcceptedUserInvitation(UserId, RemoteUserId);
				}
				else
				{
					UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncEventRailInviteResponse: No session interface"));
				}
			}
			else
			{
				UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncEventRailInviteResponse: No user logged in"));
			}
		}
		else
		{
			UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncEventRailInviteResponse: No identity interface"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncEventRailInviteResponse: Invite not accepted %d"), static_cast<int32>(Response));
	}
}

void FOnlineAsyncEventRailInviteResponse::TriggerDelegates()
{
	// noop, handled by QueryAcceptedUserInvitation()
	// ignoring LoggedOut state above or possibility that OSS interfaces are missing
}

void FOnlineAsyncEventRailJoinGameByUser::Finalize()
{
	IOnlineIdentityPtr IdentityInt = Subsystem->GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		FUniqueNetIdRailPtr UserId = StaticCastSharedPtr<const FUniqueNetIdRail>(GetFirstSignedInUser(IdentityInt));
		if (UserId.IsValid() && IdentityInt->GetLoginStatus(*UserId) == ELoginStatus::LoggedIn)
		{
			FUniqueNetIdRailRef RemoteUserId = FUniqueNetIdRail::Create(UserToJoin);

			FOnlineSessionTencentRailPtr SessionInt = StaticCastSharedPtr<FOnlineSessionTencentRail>(Subsystem->GetSessionInterface());
			if (SessionInt.IsValid())
			{
				SessionInt->QueryAcceptedUserInvitation(UserId.ToSharedRef(), RemoteUserId);
			}
			else
			{
				UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncEventRailInviteResponse: No session interface"));
			}
		}
		else
		{
			UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncEventRailJoinGameByUser: No user logged in"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Log, TEXT("FOnlineAsyncEventRailInviteResponse: No identity interface"));
	}
}

void FOnlineAsyncEventRailJoinGameByUser::TriggerDelegates()
{
	// noop, handled by QueryAcceptedUserInvitation()
	// ignoring LoggedOut state above or possibility that OSS interfaces are missing
}

void FOnlineAsyncEventRailJoinGameResult::Finalize()
{
	// nothing to do locally now that we know the remote user will be joining our session
}

void FOnlineAsyncEventRailJoinGameResult::TriggerDelegates()
{
	// nothing to do locally now that we know the remote user will be joining our session
}

void FOnlineAsyncEventRailFriendsListChanged::TriggerDelegates()
{
	IOnlineIdentityPtr IdentityInt = Subsystem->GetIdentityInterface();
	FOnlineFriendsTencentPtr FriendsInt = StaticCastSharedPtr<FOnlineFriendsTencent>(Subsystem->GetFriendsInterface());
	if (IdentityInt.IsValid() && FriendsInt.IsValid())
	{
		FUniqueNetIdRailPtr UniqueNetId = StaticCastSharedPtr<const FUniqueNetIdRail>(IdentityInt->CreateUniquePlayerId(reinterpret_cast<uint8*>(&UserId), sizeof(UserId)));
		if (UniqueNetId.IsValid())
		{
			ELoginStatus::Type LoginStatus = IdentityInt->GetLoginStatus(*UniqueNetId);
			if (LoginStatus == ELoginStatus::LoggedIn)
			{
				FriendsInt->OnRailFriendsListChanged(*UniqueNetId);
			}
			else
			{
				UE_LOG_ONLINE(Verbose, TEXT("FOnlineAsyncEventRailFriendsListChanged: User %s is not logged in, status is %s"), *UniqueNetId->ToDebugString(), ELoginStatus::ToString(LoginStatus));
			}
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncEventRailFriendsListChanged: Failed to create a unique player id for rail id %llu"), UserId.get_id());
		}
	}
	else
	{
		UE_LOG_ONLINE(Verbose, TEXT("FOnlineAsyncEventRailFriendsListChanged: Missing identity or friends interface"));
	}
}

void FOnlineAsyncEventRailFriendsOnlineStateChanged::Finalize()
{
	if (OnlineState.friend_rail_id != rail::kInvalidRailId)
	{
		IOnlinePresencePtr PresenceInt = Subsystem->GetPresenceInterface();
		if (PresenceInt.IsValid())
		{
			FOnlinePresenceTencentPtr TencentPresenceInt = StaticCastSharedPtr<FOnlinePresenceTencent>(PresenceInt);
			FUniqueNetIdRef UserId = FUniqueNetIdRail::Create(OnlineState.friend_rail_id);
			TencentPresenceInt->SetUserOnlineState(*UserId, RailOnlineStateToOnlinePresence(OnlineState.friend_online_state));
		}
	}
}

FOnlineAsyncEventRailFriendsMetadataChanged::FOnlineAsyncEventRailFriendsMetadataChanged(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::RailFriendsMetadataChanged* InParams)
	: FOnlineAsyncEventRail(InSubsystem)
{
	if (InParams)
	{
		for (uint32 Index = 0; Index < InParams->friends_changed_metadata.size(); ++Index)
		{
			const rail::RailFriendMetadata& ChangedFriend = InParams->friends_changed_metadata[Index];
			if (ChangedFriend.friend_rail_id != rail::kInvalidRailId &&
				ChangedFriend.metadatas.size() > 0)
			{
				FMetadataPropertiesRail Metadata;
				FillMetadataFromRailResult(ChangedFriend, Metadata);
				if (Metadata.Num() > 0)
				{
					ChangedMetadata.Emplace(FUniqueNetIdRail::Create(ChangedFriend.friend_rail_id), Metadata);
				}
			}
		}
	}
}

void FOnlineAsyncEventRailFriendsMetadataChanged::TriggerDelegates()
{
	for (const TPair<FUniqueNetIdRef, FMetadataPropertiesRail>& FriendMetadata : ChangedMetadata)
	{
		UE_LOG_ONLINE(Verbose, TEXT("FriendsMetadataChanged: User: %s Count: %d"), *FriendMetadata.Key->ToDebugString(), FriendMetadata.Value.Num());
		if (UE_LOG_ACTIVE(LogOnline, VeryVerbose))
		{
			for (const TPair<FString, FVariantData>& Changes : FriendMetadata.Value)
			{
				UE_LOG_ONLINE(VeryVerbose, TEXT(" - %s : %s"), *Changes.Key, *Changes.Value.ToString());
			}
		}
		Subsystem->TriggerOnFriendMetadataChangedDelegates(*FriendMetadata.Key, FriendMetadata.Value);
	}
}

void FOnlineAsyncTaskRailQueryFriendsPresence::Initialize()
{
	IOnlinePresencePtr PresenceInt = Subsystem->GetPresenceInterface();
	if (PresenceInt.IsValid())
	{
		IOnlinePresence::FOnPresenceTaskCompleteDelegate UserCompletionDelegate;
		UserCompletionDelegate.BindLambda([this](const FUniqueNetId& User, bool bInWasSuccessful)
		{
			TaskResult.QueryCount++;
			TaskResult.SuccessCount += (bInWasSuccessful ? 1 : 0);
			if (TaskResult.QueryCount >= FriendsList.Num())
			{
				TaskResult.Error.bSucceeded = (TaskResult.SuccessCount == FriendsList.Num());
				if (!TaskResult.Error.WasSuccessful())
				{
					TaskResult.Error.SetFromErrorCode(TEXT("railPresence.query_friends_failure"));
				}
				bWasSuccessful = TaskResult.Error.WasSuccessful();
				bIsComplete = true;
			}
		});

		for (const FUniqueNetIdRef& Friend : FriendsList)
		{
			PresenceInt->QueryPresence(*Friend, UserCompletionDelegate);
		}
	}
}

void FOnlineAsyncTaskRailQueryFriendsPresence::Finalize()
{
}

void FOnlineAsyncTaskRailQueryFriendsPresence::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

void FOnlineAsyncTaskRailSendInvite::Initialize()
{
	rail::RailResult Result = rail::RailResult::kErrorUnknown;
	if (rail::IRailUsersHelper* const UsersHelper = RailSdkWrapper::Get().RailUsersHelper())
	{
		FOnlineAsyncTaskRail::Initialize();

		RailInviteOptions Options;
		Options.invite_type = EnumRailUsersInviteType::kRailUsersInviteTypeGame;

		rail::RailArray<rail::RailID> RailIds;
		for (const FUniqueNetIdRef& UserId : UserIds)
		{
			RailIds.push_back(StaticCastSharedRef<const FUniqueNetIdRail>(UserId)->RailID);
		}
		rail::RailString RailInviteStr;
		ToRailString(TaskResult.InviteStr, RailInviteStr);
		Result = UsersHelper->AsyncInviteUsers(RailInviteStr, RailIds, Options, RailString());
	}

	if (Result != rail::RailResult::kSuccess)
	{
		ParseRailResult(Result, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailSendInvite::Finalize()
{
	FOnlineAsyncTaskRail::Finalize();
}
void FOnlineAsyncTaskRailSendInvite::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

void FOnlineAsyncTaskRailSendInvite::OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailSendInvite::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventUsersInviteUsersResult:
			OnRailEventFriendsSendInviteResult(static_cast<rail_event::RailUsersInviteUsersResult*>(param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailSendInvite::OnRailEventFriendsSendInviteResult(const rail::rail_event::RailUsersInviteUsersResult* InviteResult)
{
	ParseRailResult(InviteResult, TaskResult.Error);
	UE_LOG_ONLINE(Verbose, TEXT("FOnlineAsyncTaskRailSendInvite::OnRailEventFriendsSendInviteResult Result: %s"), *TaskResult.Error.ToLogString());

	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

FOnlineAsyncTaskRailGetInviteDetails::FOnlineAsyncTaskRailGetInviteDetails(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const FOnOnlineAsyncTaskRailGetInviteDetailsComplete& InCompletionDelegate)
	: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventUsersGetInviteDetailResult})
	, RailUserId((rail::RailID)InUserId)
	, CompletionDelegate(InCompletionDelegate)
{
}

void FOnlineAsyncTaskRailGetInviteDetails::Initialize()
{
	rail::RailResult Result = rail::RailResult::kErrorUnknown;
	if (rail::IRailUsersHelper* const UsersHelper = RailSdkWrapper::Get().RailUsersHelper())
	{
		FOnlineAsyncTaskRail::Initialize();
		Result = UsersHelper->AsyncGetInviteDetail(RailUserId, rail::kRailUsersInviteTypeGame, RailString());
	}

	if (Result != rail::RailResult::kSuccess)
	{
		ParseRailResult(Result, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}
void FOnlineAsyncTaskRailGetInviteDetails::Finalize()
{
	FOnlineAsyncTaskRail::Finalize();
}
void FOnlineAsyncTaskRailGetInviteDetails::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

void FOnlineAsyncTaskRailGetInviteDetails::OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailGetInviteDetails::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventUsersGetInviteDetailResult:
			OnRailEventFriendsGetInviteDetailsResult(static_cast<rail_event::RailUsersGetInviteDetailResult*>(param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailGetInviteDetails::OnRailEventFriendsGetInviteDetailsResult(const rail::rail_event::RailUsersGetInviteDetailResult* InviteDetails)
{
	ParseRailResult(InviteDetails, TaskResult.Error);
	UE_LOG_ONLINE(Log, TEXT("OnRailEventFriendsGetInviteDetailsResult Result: %s"), *TaskResult.Error.ToLogString());

	if (InviteDetails)
	{
		if (RailUserId == InviteDetails->inviter_id &&
			InviteDetails->invite_type == EnumRailUsersInviteType::kRailUsersInviteTypeGame)
		{
			TaskResult.InviteString = LexToString(InviteDetails->command_line);
		}
		else
		{
			const FUniqueNetIdRailRef RailInviterId = FUniqueNetIdRail::Create(InviteDetails->inviter_id);
			UE_LOG_ONLINE(Verbose, TEXT("Unsupported or invalid invite details %s %s"), *RailInviterId->ToDebugString(), *LexToString(InviteDetails->invite_type));
		}
	}

	bWasSuccessful = TaskResult.Error.WasSuccessful() && !TaskResult.InviteString.IsEmpty();
	bIsComplete = true;
}

FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo::FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const FOnOnlineAsyncTaskRailQueryFriendPlayedGamesComplete& InCompletionDelegate)
	: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventFriendsGetFriendPlayedGamesResult})
	, CompletionDelegate(InCompletionDelegate)
{
	TaskResult.FriendId = StaticCastSharedRef<const FUniqueNetIdRail>(InUserId.AsShared());
}

void FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo::Initialize()
{
	rail::RailResult Result = rail::RailResult::kErrorUnknown;
	if (rail::IRailFriends* const RailFriends = RailSdkWrapper::Get().RailFriends())
	{
		FOnlineAsyncTaskRail::Initialize();
		Result = RailFriends->AsyncQueryFriendPlayedGamesInfo((rail::RailID)(*TaskResult.FriendId), RailString());
	}

	if (Result != rail::RailResult::kSuccess)
	{
		ParseRailResult(Result, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo::Finalize()
{
	FOnlineAsyncTaskRail::Finalize();
}

void FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo::TriggerDelegates()
{ 
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

void FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo::OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param)
{ 
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventFriendsGetFriendPlayedGamesResult:
			OnRailEventQueryFriendPlayedGamesResult(static_cast<rail_event::RailFriendsQueryFriendPlayedGamesResult *>(param));
			break;
		default:
			break;
	}
}
	
void FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo::OnRailEventQueryFriendPlayedGamesResult(const rail::rail_event::RailFriendsQueryFriendPlayedGamesResult* PlayedGamesDetail)
{
	ParseRailResult(PlayedGamesDetail, TaskResult.Error);
	UE_LOG_ONLINE(Log, TEXT("OnRailEventQueryFriendPlayedGamesResult Result: %s"), *TaskResult.Error.ToLogString());

	if (PlayedGamesDetail)
	{
		for (uint32 Index = 0; Index < PlayedGamesDetail->friend_played_games_info_list.size(); ++Index)
		{
			const RailFriendPlayedGameInfo& PlayedGameInfo = PlayedGamesDetail->friend_played_games_info_list[Index];
			if ((rail::RailID)(*TaskResult.FriendId) == PlayedGameInfo.friend_id)
			{
				FQueryFriendPlayedGamesTaskResult::FRailGamePlayedInfo GameInfo;
				GameInfo.bInServer = PlayedGameInfo.in_game_server;
				GameInfo.bInRoom = PlayedGameInfo.in_room;
				// NYI ServerId and RoomId support

				GameInfo.GameId = PlayedGameInfo.game_id;
				GameInfo.GamePlayState = PlayedGameInfo.friend_played_game_play_state;

				TaskResult.GameInfos.Emplace(GameInfo);
			}
			else
			{
				UE_LOG_ONLINE(Verbose, TEXT("OnRailEventQueryFriendPlayedGamesResult friend id doesn't match"));
			}
		}
	}

	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

FOnlineAsyncTaskRailReportPlayedWithUsers::FOnlineAsyncTaskRailReportPlayedWithUsers(FOnlineSubsystemTencent* InSubsystem, const TArray<FReportPlayedWithUser>& InUsersReported, const FOnOnlineAsyncTaskRailReportPlayedWithUsersComplete& InCompletionDelegate)
	: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventFriendsReportPlayedWithUserListResult})
	, CompletionDelegate(InCompletionDelegate)
{
	TaskResult.UsersReported.Append(InUsersReported);
}

void FOnlineAsyncTaskRailReportPlayedWithUsers::Initialize()
{
	rail::RailResult Result = rail::RailResult::kErrorUnknown;
	if (TaskResult.UsersReported.Num() > 0)
	{
		if (TaskResult.UsersReported.Num() < kRailMaxPlayedWithUsersCount)
		{
			if (rail::IRailFriends* const RailFriends = RailSdkWrapper::Get().RailFriends())
			{
				FOnlineAsyncTaskRail::Initialize();

				rail::RailArray<rail::RailUserPlayedWith> RailPlayedWith(TaskResult.UsersReported.Num());
				for (const FReportPlayedWithUser& UserReport : TaskResult.UsersReported)
				{
					rail::RailUserPlayedWith PlayedWith;
					PlayedWith.rail_id = StaticCastSharedRef<const FUniqueNetIdRail>(UserReport.UserId)->RailID;
					if (UserReport.PresenceStr.Len() < kRailMaxPlayedWithUserContentLen)
					{
						ToRailString(UserReport.PresenceStr, PlayedWith.user_rich_content);
					}

					RailPlayedWith.push_back(PlayedWith);
				}

				Result = RailFriends->AsyncReportPlayedWithUserList(RailPlayedWith, RailString());
			}
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("Too many users to report to RailSDK %d"), TaskResult.UsersReported.Num());
		}
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("Report users called with empty user list"));
	}

	if (Result != rail::RailResult::kSuccess)
	{
		ParseRailResult(Result, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailReportPlayedWithUsers::Finalize()
{
	FOnlineAsyncTaskRail::Finalize();
}

void FOnlineAsyncTaskRailReportPlayedWithUsers::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

void FOnlineAsyncTaskRailReportPlayedWithUsers::OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailReportPlayedWithUsers::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventFriendsReportPlayedWithUserListResult:
			OnRailEventReportPlayedWithUsers(static_cast<rail_event::RailFriendsReportPlayedWithUserListResult *>(param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailReportPlayedWithUsers::OnRailEventReportPlayedWithUsers(const rail::rail_event::RailFriendsReportPlayedWithUserListResult* PlayedWithUsersResult)
{
	ParseRailResult(PlayedWithUsersResult, TaskResult.Error);
	UE_LOG_ONLINE(Log, TEXT("OnRailEventReportPlayedWithUsers Result: %s"), *TaskResult.Error.ToLogString());

	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

FOnlineAsyncTaskRailQueryPlayedWithFriendsList::FOnlineAsyncTaskRailQueryPlayedWithFriendsList(FOnlineSubsystemTencent* InSubsystem, const FOnOnlineAsyncTaskRailQueryPlayedWithFriendsListComplete& InCompletionDelegate)
	: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventFriendsQueryPlayedWithFriendsListResult})
	, CompletionDelegate(InCompletionDelegate)
{
	
}

void FOnlineAsyncTaskRailQueryPlayedWithFriendsList::Initialize()
{
	rail::RailResult Result = rail::RailResult::kErrorUnknown;

	if (rail::IRailFriends* const RailFriends = RailSdkWrapper::Get().RailFriends())
	{
		FOnlineAsyncTaskRail::Initialize();
		Result = RailFriends->AsyncQueryPlayedWithFriendsList(RailString());
	}
	
	if (Result != rail::RailResult::kSuccess)
	{
		ParseRailResult(Result, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailQueryPlayedWithFriendsList::Finalize()
{
	FOnlineAsyncTaskRail::Finalize();
}

void FOnlineAsyncTaskRailQueryPlayedWithFriendsList::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

void FOnlineAsyncTaskRailQueryPlayedWithFriendsList::OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailQueryPlayedWithFriendsList::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventFriendsQueryPlayedWithFriendsListResult:
			OnRailEventPlayedWithFriendsList(static_cast<rail_event::RailFriendsQueryPlayedWithFriendsListResult*>(param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailQueryPlayedWithFriendsList::OnRailEventPlayedWithFriendsList(const rail::rail_event::RailFriendsQueryPlayedWithFriendsListResult* PlayedFriendsList)
{
	ParseRailResult(PlayedFriendsList, TaskResult.Error);
	UE_LOG_ONLINE(Log, TEXT("OnRailEventPlayedWithFriendsList Result: %s"), *TaskResult.Error.ToLogString());

	if (PlayedFriendsList)
	{
		for (uint32 RailIdx = 0; RailIdx < PlayedFriendsList->played_with_friends_list.size(); ++RailIdx)
		{
			if (PlayedFriendsList->played_with_friends_list[RailIdx] != kInvalidRailId)
			{
				TaskResult.UsersPlayedWith.Add(FUniqueNetIdRail::Create(PlayedFriendsList->played_with_friends_list[RailIdx]));
			}
		}
	}

	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

FOnlineAsyncTaskRailQueryPlayedWithFriendsTime::FOnlineAsyncTaskRailQueryPlayedWithFriendsTime(FOnlineSubsystemTencent* InSubsystem, const TArray<FUniqueNetIdRef>& InUserIds, const FOnOnlineAsyncTaskRailQueryPlayedWithFriendsTimeComplete& InCompletionDelegate)
	: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventFriendsQueryPlayedWithFriendsTimeResult})
	, CompletionDelegate(InCompletionDelegate)
{ 
	TaskResult.UserIds.Append(InUserIds);
}

void FOnlineAsyncTaskRailQueryPlayedWithFriendsTime::Initialize() 
{
	rail::RailResult Result = rail::RailResult::kErrorUnknown;

	if (rail::IRailFriends* const RailFriends = RailSdkWrapper::Get().RailFriends())
	{
		FOnlineAsyncTaskRail::Initialize();

		rail::RailArray<rail::RailID> RailIds;
		for (const FUniqueNetIdRef& UserId : TaskResult.UserIds)
		{
			RailIds.push_back(StaticCastSharedRef<const FUniqueNetIdRail>(UserId)->RailID);
		}

		Result = RailFriends->AsyncQueryPlayedWithFriendsTime(RailIds, RailString());
	}

	if (Result != rail::RailResult::kSuccess)
	{
		ParseRailResult(Result, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailQueryPlayedWithFriendsTime::Finalize()
{
	FOnlineAsyncTaskRail::Finalize();
}

void FOnlineAsyncTaskRailQueryPlayedWithFriendsTime::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

void FOnlineAsyncTaskRailQueryPlayedWithFriendsTime::OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) 
{ 
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailQueryPlayedWithFriendsTime::OnRailEvent"));
	switch (event_id)
	{
		case rail::kRailEventFriendsQueryPlayedWithFriendsTimeResult:
			OnRailEventPlayedWithFriendsTime(static_cast<rail_event::RailFriendsQueryPlayedWithFriendsTimeResult *>(param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailQueryPlayedWithFriendsTime::OnRailEventPlayedWithFriendsTime(const rail::rail_event::RailFriendsQueryPlayedWithFriendsTimeResult* PlayedTimeResult)
{
	ParseRailResult(PlayedTimeResult, TaskResult.Error);
	UE_LOG_ONLINE(Log, TEXT("OnRailEventPlayedWithFriendsTime Result: %s"), *TaskResult.Error.ToLogString());

	if (PlayedTimeResult)
	{
		for (uint32 RailIdx = 0; RailIdx < PlayedTimeResult->played_with_friends_time_list.size(); ++RailIdx)
		{
			const RailPlayedWithFriendsTimeItem& FriendItem = PlayedTimeResult->played_with_friends_time_list[RailIdx];
			if (FriendItem.rail_id != kInvalidRailId)
			{
				TaskResult.LastPlayedWithUsers.Emplace(FUniqueNetIdRail::Create(FriendItem.rail_id), FDateTime::FromUnixTimestamp(FriendItem.play_time));
			}
		}
	}

	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

void FOnlineAsyncTaskRailRequestAllAssets::Initialize()
{
	rail::RailResult Result = rail::RailResult::kErrorUnknown;
	if (rail::IRailAssets* const RailAssets = RailSdkWrapper::Get().RailAssets())
	{
		FOnlineAsyncTaskRail::Initialize();
		rail::RailString UserData;
		Result = RailAssets->AsyncRequestAllAssets(UserData);
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskRailRequestAllAssets: Could not find IRailAssets"));
	}

	if (Result != rail::RailResult::kSuccess)
	{
		ParseRailResult(Result, TaskResult.Error);
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailRequestAllAssets::TriggerDelegates()
{
	FOnlineAsyncTaskRail::TriggerDelegates();
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

void FOnlineAsyncTaskRailRequestAllAssets::OnRailEvent(rail::RAIL_EVENT_ID EventId, rail::EventBase* Param)
{
	UE_LOG_ONLINE(Verbose, TEXT("FOnlineAsyncTaskRailRequestAllAssets::OnRailEvent"));
	switch (EventId)
	{
		case rail::kRailEventAssetsRequestAllAssetsFinished:
			OnRailRequestAllAssetsFinished(static_cast<rail_event::RequestAllAssetsFinished*>(Param));
			break;
	}
}

void FOnlineAsyncTaskRailRequestAllAssets::OnRailRequestAllAssetsFinished(const rail::rail_event::RequestAllAssetsFinished* AssetsResponse)
{
	ParseRailResult(AssetsResponse, TaskResult.Error);
	if (TaskResult.Error.WasSuccessful())
	{
		const rail::RailArray<rail::RailAssetInfo>& AssetInfos(AssetsResponse->assetinfo_list);
		UE_LOG_ONLINE(VeryVerbose, TEXT("RequestAllAssets: %u assets"), AssetInfos.size());
		for (uint32 AssetIndex = 0; AssetIndex < AssetInfos.size(); ++AssetIndex)
		{
			const rail::RailAssetInfo& AssetInfo(AssetInfos[AssetIndex]);
			UE_LOG_ONLINE_PURCHASE(VeryVerbose, TEXT(" %u: %s"), AssetIndex, *RailAssetInfoToJsonString(AssetInfo));
		}

		RailAssetInfosToPurchaseReceipt(*UserId, AssetInfos, TaskResult.PurchaseReceipt);
	}

	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

FOnlineAsyncEventRailAssetsChanged::FOnlineAsyncEventRailAssetsChanged(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::RailAssetsChanged* InParams)
	: FOnlineAsyncEventRail(InSubsystem)
{
	// RailAssetsChanged has no data
}

void FOnlineAsyncEventRailAssetsChanged::TriggerDelegates()
{
	IOnlineIdentityPtr IdentityInt = Subsystem->GetIdentityInterface();
	FOnlinePurchaseTencentPtr PurchaseInt = StaticCastSharedPtr<FOnlinePurchaseTencent>(Subsystem->GetPurchaseInterface());
	if (LIKELY(IdentityInt.IsValid() && PurchaseInt.IsValid()))
	{
		const FUniqueNetIdPtr UniqueNetId = GetFirstSignedInUser(IdentityInt);
		if (UniqueNetId.IsValid())
		{
			const ELoginStatus::Type LoginStatus = IdentityInt->GetLoginStatus(*UniqueNetId);
			if (LoginStatus == ELoginStatus::LoggedIn)
			{
				PurchaseInt->OnRailAssetsChanged(*UniqueNetId);
			}
			else
			{
				UE_LOG_ONLINE(Verbose, TEXT("FOnlineAsyncEventRailAssetsChanged: User %s is not logged in, status is %s"), *UniqueNetId->ToDebugString(), ELoginStatus::ToString(LoginStatus));
			}
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncEventRailAssetsChanged: No signed in user"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Verbose, TEXT("FOnlineAsyncEventRailAssetsChanged: Missing identity or purchase interface"));
	}
}

void FOnlineAsyncTaskRailRequestAllPurchasableProducts::Initialize()
{
	UE_LOG_ONLINE(Verbose, TEXT("FOnlineAsyncTaskRailSetSessionMetadata::Initialize"));

	rail::IRailInGamePurchase* const RailInGamePurchase = RailSdkWrapper::Get().RailInGamePurchase();
	if (RailInGamePurchase)
	{
		FOnlineAsyncTaskRail::Initialize();
		rail::RailString UserData;
		const rail::RailResult RailResult = RailInGamePurchase->AsyncRequestAllPurchasableProducts(UserData);
		ParseRailResult(RailResult, TaskResult.Error);
	}
	else
	{
		TaskResult.Error.SetFromErrorCode(TEXT("missing RailInGamePurchase"));
	}

	if (!TaskResult.Error.WasSuccessful())
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskRailRequestAllPurchasableProducts::Initialize failed with result=[%s]"), *TaskResult.Error.ToLogString());
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailRequestAllPurchasableProducts::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

void FOnlineAsyncTaskRailRequestAllPurchasableProducts::OnRailEvent(rail::RAIL_EVENT_ID EventId, rail::EventBase* Param)
{
	UE_LOG_ONLINE(Verbose, TEXT("FOnlineAsyncTaskRailRequestAllPurchasableProducts::OnRailEvent"));
	switch (EventId)
	{
		case rail::kRailEventInGamePurchaseAllPurchasableProductsInfoReceived:
			OnRailRequestAllPurchasableProductsResponse(static_cast<rail_event::RailInGamePurchaseRequestAllPurchasableProductsResponse*>(Param));
			break;
		default:
			break;
	}
}

static int32 AdjustPriceForLocalization(const FString& InCurrencyCode, const double InPrice)
{
	// Convert the backend stated price into its base units
	FInternationalization& I18N = FInternationalization::Get();
	const FCulture& Culture = *I18N.GetCurrentCulture();

	const FDecimalNumberFormattingRules& FormattingRules = Culture.GetCurrencyFormattingRules(InCurrencyCode);
	const FNumberFormattingOptions& FormattingOptions = FormattingRules.CultureDefaultFormattingOptions;
	double FormattedPrice = InPrice * static_cast<double>(FMath::Pow(10.0f, FormattingOptions.MaximumFractionalDigits));
	return FMath::TruncToInt(FormattedPrice + 0.5);
}

void FOnlineAsyncTaskRailRequestAllPurchasableProducts::OnRailRequestAllPurchasableProductsResponse(rail::rail_event::RailInGamePurchaseRequestAllPurchasableProductsResponse* Response)
{
	ParseRailResult(Response, TaskResult.Error);
	if (TaskResult.Error.WasSuccessful())
	{
		const FDateTime UtcNow = FDateTime::UtcNow();
		TaskResult.Offers.Reserve(Response->purchasable_products.size());
		for (int32 Index = 0; Index < Response->purchasable_products.size(); ++Index)
		{
			const rail::RailPurchaseProductInfo& ProductInfo(Response->purchasable_products[Index]);
			
			FOnlineStoreOfferRef StoreOffer = MakeShared<FOnlineStoreOffer>();
			static_assert(std::is_same_v<decltype(ProductInfo.product_id), uint32>, "Change how we construct string from ProductInfo.product_id");
			StoreOffer->OfferId = FString::Printf(TEXT("%u"), ProductInfo.product_id);
			StoreOffer->Title = FText::FromString(LexToString(ProductInfo.name));
			StoreOffer->Description = FText::FromString(LexToString(ProductInfo.description));
			StoreOffer->LongDescription = StoreOffer->Description;
			StoreOffer->CurrencyCode = LexToString(ProductInfo.currency_type);
			StoreOffer->RegularPrice = AdjustPriceForLocalization(StoreOffer->CurrencyCode, ProductInfo.original_price);

			switch (ProductInfo.discount.type)
			{
			case kPurchaseProductDiscountTypeInvalid:
				break;
			case kPurchaseProductDiscountTypeNone:
				StoreOffer->NumericPrice = AdjustPriceForLocalization(StoreOffer->CurrencyCode, ProductInfo.original_price);
				break;
			case kPurchaseProductDiscountTypePermanent:
				StoreOffer->NumericPrice = AdjustPriceForLocalization(StoreOffer->CurrencyCode, ProductInfo.discount.discount_price);
				break;
			case kPurchaseProductDiscountTypeTimed:
			{
				const FDateTime UtcStartTime = FDateTime::FromUnixTimestamp(ProductInfo.discount.start_time);
				const FDateTime UtcEndTime = FDateTime::FromUnixTimestamp(ProductInfo.discount.end_time);
				if (UtcNow >= UtcStartTime && UtcNow <= UtcEndTime)
				{
					StoreOffer->NumericPrice = AdjustPriceForLocalization(StoreOffer->CurrencyCode, ProductInfo.discount.discount_price);
				}
				else
				{
					StoreOffer->NumericPrice = AdjustPriceForLocalization(StoreOffer->CurrencyCode, ProductInfo.original_price);
				}

				if (UtcNow < UtcStartTime)
				{
					StoreOffer->ExpirationDate = UtcStartTime;
				}
				else if (UtcNow < UtcEndTime)
				{
					StoreOffer->ExpirationDate = UtcEndTime;
				}
				break;
			}
			}
			TaskResult.Offers.Emplace(StoreOffer);
		}
	}
	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

void FOnlineAsyncTaskRailPurchaseProducts::Initialize()
{
	UE_LOG_ONLINE(Verbose, TEXT("FOnlineAsyncTaskRailPurchaseProducts::Initialize"));

	rail::IRailInGamePurchase* const RailInGamePurchase = RailSdkWrapper::Get().RailInGamePurchase();
	if (RailInGamePurchase)
	{
		FOnlineAsyncTaskRail::Initialize();
		
		rail::RailArray<rail::RailProductItem> RailProductItems;
		for (const FPurchaseCheckoutRequest::FPurchaseOfferEntry& OfferEntry : CheckoutRequest.PurchaseOffers)
		{
			rail::RailProductItem ProductItem;
			ProductItem.product_id = FCString::Atoi(*OfferEntry.OfferId);
			ProductItem.quantity = OfferEntry.Quantity;
			RailProductItems.push_back(ProductItem);
		}

		rail::RailString UserData;
		const rail::RailResult RailResult = RailInGamePurchase->AsyncPurchaseProductsToAssets(RailProductItems, UserData);
		if (RailResult == rail::kSuccess)
		{
			TaskResult.Error.bSucceeded = true;
		}
		else
		{
			HandlePurchaseProductsResult(RailResult);
		}
	}
	else
	{
		TaskResult.Error.SetFromErrorCode(TEXT("com.epicgames.purchase.missing_ingamepurchase"));
	}

	if (!TaskResult.Error.bSucceeded)
	{
		UE_LOG_ONLINE(Warning, TEXT("FOnlineAsyncTaskRailPurchaseProducts::Initialize failed with result=[%s]"), *TaskResult.Error.ToLogString());
		bWasSuccessful = TaskResult.Error.WasSuccessful();
		bIsComplete = true;
	}
}

void FOnlineAsyncTaskRailPurchaseProducts::TriggerDelegates()
{
	CompletionDelegate.ExecuteIfBound(TaskResult);
}

void FOnlineAsyncTaskRailPurchaseProducts::OnRailEvent(rail::RAIL_EVENT_ID EventId, rail::EventBase* Param)
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineAsyncTaskRailPurchaseProducts::OnRailEvent"));
	switch (EventId)
	{
		case rail::kRailEventInGamePurchasePurchaseProductsToAssetsResult:
			OnRailPurchaseProductsToAssetsResult(static_cast<rail_event::RailInGamePurchasePurchaseProductsToAssetsResponse*>(Param));
			break;
		default:
			break;
	}
}

void FOnlineAsyncTaskRailPurchaseProducts::HandlePurchaseProductsResult(const TOptional<rail::RailResult> RailResult)
{
	TaskResult.Error.bSucceeded = false;
	if (RailResult.IsSet())
	{
		TaskResult.Error.ErrorRaw = FString::Printf(TEXT("0x%08x"), static_cast<int32>(RailResult.GetValue()));
		switch (RailResult.GetValue())
		{
		case rail::kErrorInGamePurchasePaymentCancle:
			// User cancelled the payment
			TaskResult.Error.SetFromErrorCode(TEXT("com.epicgames.catalog_helper.user_cancelled"));
			break;

		case rail::kErrorInGamePurchaseProductInfoExpired: // Intentional fall-through
		case rail::kErrorInGamePurchaseProductIsNotExist:
			// Offers are out of date
			TaskResult.Error.SetFromErrorCode(TEXT("com.epicgames.purchase.offers_out_of_date"));
			break;

		case rail::kErrorInGamePurchasePaymentFailed: // Intentional fall-through
		case rail::kErrorInGamePurchaseAcquireSessionTicketFailed: // Intentional fall-through
		case rail::kErrorInGamePurchaseParseWebContentFaild: // Intentional fall-through
		case rail::kErrorInGamePurchaseOrderIDIsNotExist: // Intentional fall-through
		case rail::kErrorInGamePurchasePreparePaymentRequestTimeout: // Intentional fall-through
		case rail::kErrorInGamePurchaseCreateOrderFailed: // Intentional fall-through
		case rail::kErrorInGamePurchaseQueryOrderFailed: // Intentional fall-through
		case rail::kErrorInGamePurchaseFinishOrderFailed: // Intentional fall-through
		case rail::kErrorInGamePurchaseCreatePaymentBrowserFailed: // Intentional fall-through
		default:
			TaskResult.Error.SetFromErrorCode(TEXT("com.epicgames.purchase.failure"));
			break;
		}
	}
	else
	{
		TaskResult.Error.SetFromErrorCode(TEXT("com.epicgames.purchase.noresponse"));
	}
}

void FOnlineAsyncTaskRailPurchaseProducts::OnRailPurchaseProductsToAssetsResult(rail::rail_event::RailInGamePurchasePurchaseProductsToAssetsResponse* Response)
{
	// Intentionally does not call ParseRailResult
	if (Response && Response->result == rail::kSuccess)
	{
		TaskResult.PurchaseReceipt = MakeShared<FPurchaseReceipt>();

		const rail::RailArray<rail::RailAssetInfo>& AssetInfos(Response->delivered_assets);
		RailAssetInfosToPurchaseReceipt(*UserId, AssetInfos, *TaskResult.PurchaseReceipt);

		FString OrderId = LexToString(Response->order_id);
		FString DeliveredAssetsLogString;
		for (uint32 AssetIndex = 0; AssetIndex < AssetInfos.size(); ++AssetIndex)
		{
			if (AssetIndex != 0)
			{
				DeliveredAssetsLogString += TEXT(",");
			}
			DeliveredAssetsLogString += RailAssetInfoToJsonString(AssetInfos[AssetIndex]);
		}
		UE_LOG_ONLINE_PURCHASE(Log, TEXT("PurchaseProductsToAssets: Successful with OrderId=[%s] DeliveredAssets=[%s]"), *OrderId, *DeliveredAssetsLogString);
	}
	else
	{
		HandlePurchaseProductsResult(Response ? Response->result : TOptional<rail::RailResult>());
	}
	bWasSuccessful = TaskResult.Error.WasSuccessful();
	bIsComplete = true;
}

#endif // WITH_TENCENT_RAIL_SDK
