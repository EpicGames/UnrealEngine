// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TENCENT_RAIL_SDK

#include "RailSDK.h"
#include "RailSdkWrapper.h"
#include "OnlineAsyncTaskManager.h"
#include "Interfaces/OnlineFriendsInterface.h"
#include "Interfaces/OnlinePresenceInterface.h"
#include "Interfaces/OnlinePurchaseInterface.h"
#include "OnlineSubsystemTencentTypes.h"
#include "OnlineError.h"
#include "Serialization/JsonSerializerMacros.h"

class FOnlineSubsystemTencent;
class FUniqueNetIdRail;
class FOnlineUserInfoTencent;
typedef TSharedRef<class FOnlineStoreOffer> FOnlineStoreOfferRef;

/** Timeout failsafe in case Tencent tasks/events don't get a callback */
#if UE_BUILD_SHIPPING
#define ASYNC_RAIL_TASK_TIMEOUT 20.0
#else
#define ASYNC_RAIL_TASK_TIMEOUT 80.0
#endif

/**
 * Base class that holds a delegate to fire when a given async task is complete
 */
class FOnlineAsyncTaskRail : public FOnlineAsyncTaskBasic<FOnlineSubsystemTencent>, public rail::IRailEvent
{

public:

	FOnlineAsyncTaskRail() = delete;
	FOnlineAsyncTaskRail(FOnlineSubsystemTencent* InSubsystem, TSet<rail::RAIL_EVENT_ID> InRegisteredRailEvents)
		: FOnlineAsyncTaskBasic(InSubsystem)
		, RegisteredRailEvents(InRegisteredRailEvents)
	{
	}

	virtual ~FOnlineAsyncTaskRail() = default;

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void Tick() override;
	//~ End FOnlineAsyncTask interface

protected:

	/**
	 * Convert a raw Rail result into something resembling an FOnlineError
	 *
	 * @param InResult any rail result
	 * @param OutOnlineError the converted error result
	 */
	void ParseRailResult(rail::RailResult InResult, FOnlineError& OutOnlineError) const;

	/**
	 * Parse a Rail event into something resembling an FOnlineError
	 *
	 * @param InResult any rail event
	 * @param OutOnlineError the converted error result
	 */
	void ParseRailResult(const rail::EventBase* InResult, FOnlineError& OutOnlineError) const;

	/**
	 * Can this task timeout?
	 * Most tasks are able to timeout.
	 */
	bool bCanTimeout = true;

	// Events registered with this task
	TSet<rail::RAIL_EVENT_ID> RegisteredRailEvents;
};

class FOnlineAsyncEventRail : public FOnlineAsyncEvent<FOnlineSubsystemTencent>
{

public:

	FOnlineAsyncEventRail() = delete;
	FOnlineAsyncEventRail(FOnlineSubsystemTencent* InSubsystem)
		: FOnlineAsyncEvent(InSubsystem)
	{
	}

	virtual ~FOnlineAsyncEventRail() = default;

	/**
	 * Convert a raw Rail result into something resembling an FOnlineError
	 *
	 * @param InResult any rail result
	 * @param OutOnlineError the converted error result
	 */
	void ParseRailResult(rail::RailResult InResult, FOnlineError& OutOnlineError) const;

	/**
	 * Parse a Rail event into something resembling an FOnlineError
	 *
	 * @param InResult any rail event
	 * @param OutOnlineError the converted error result
	 */
	void ParseRailResult(const rail::EventBase* InResult, FOnlineError& OutOnlineError) const;
};

class FOnlineAsyncTaskRailAcquireSessionTicket
	: public FOnlineAsyncTaskRail
{

public:
	DECLARE_DELEGATE_TwoParams(FCompletionDelegate, const FOnlineError& /*Result*/, const FString& /*SessionTicket*/);

	FOnlineAsyncTaskRailAcquireSessionTicket() = delete;
	FOnlineAsyncTaskRailAcquireSessionTicket(FOnlineSubsystemTencent* InSubsystem, rail::RailID InPlayerId, const FCompletionDelegate& InCompletionDelegate)
		: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventSessionTicketGetSessionTicket})
		, PlayerId(InPlayerId)
		, CompletionDelegate(InCompletionDelegate)
	{
	}

	virtual ~FOnlineAsyncTaskRailAcquireSessionTicket()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailAcquireSessionTicket OnlineError: %s"), *OnlineError.ToLogString()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID EventId, rail::EventBase* Param) override;
	//~ End IRailEvent interface

protected:

	void OnRailGetSessionTicket(const rail::rail_event::AcquireSessionTicketResponse* SessionTicketResponse);

	/** Player id we are obtaining the auth session for */
	rail::RailID PlayerId;
	/** The result */
	FOnlineError OnlineError;
	/** The acquired session ticket */
	FString SessionTicket;
	/** Function to fire when complete */
	FCompletionDelegate CompletionDelegate;
};

/**
* Result structure after opening/closing a floating window
*/
struct FFloatingWindowTaskResult
{
	FFloatingWindowTaskResult()
		: bOpened(false)
	{
	}

	/** Error information */
	FOnlineError Error;
	/** State of the floating window */
	bool bOpened;
};

class FOnlineAsyncTaskRailShowFloatingWindow 
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailShowFloatingWindow() = delete;
	FOnlineAsyncTaskRailShowFloatingWindow(FOnlineSubsystemTencent* InSubsystem, rail::EnumRailWindowType InWindowType)
		: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventShowFloatingWindow})
		, WindowType(InWindowType)
	{
	}

	virtual ~FOnlineAsyncTaskRailShowFloatingWindow()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailShowFloatingWindow bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:

	void OnRailShowFloatingWindow(rail::rail_event::ShowFloatingWindowResult* param);

	/** Result of the task */
	FFloatingWindowTaskResult TaskResult;
	/** Type of window to open */
	rail::EnumRailWindowType WindowType;
};

/**
 * Result structure after setting user metadata / presence
 */
struct FSetUserMetadataTaskResult
{
	/** Error information */
	FOnlineError Error;
	/** Key/Value pairs in string form used with this task */
	TMap<FString, FString> FinalData;
};

/**
 * Delegate fired when FOnlineAsyncTaskRailSetUserMetadata completes
 *
 * @param Result the result of the call
 */
DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailSetUserMetadataComplete, const FSetUserMetadataTaskResult& /*Result*/);

class FOnlineAsyncTaskRailSetUserMetadata
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailSetUserMetadata() = delete;
	FOnlineAsyncTaskRailSetUserMetadata(FOnlineSubsystemTencent* InSubsystem, const FMetadataPropertiesRail& InMetdata, const FOnOnlineAsyncTaskRailSetUserMetadataComplete& InCompletionDelegate)
		: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventFriendsSetMetadataResult})
		, Metadata(InMetdata)
		, CompletionDelegate(InCompletionDelegate)
	{
	}

	virtual ~FOnlineAsyncTaskRailSetUserMetadata()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailSetUserMetadata bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:

	/**
	 * Given an input of local metadata, populate a list of key value pairs for consumption by RailSDK
	 *
	 * @param InMetadata local metadata to convert to RailSDK
	 * @param OutFinalData map of data in final format
	 * @param OutRailKeyValuePairs list of keys and their values
	 */
	virtual bool GenerateRailMetadata(FMetadataPropertiesRail& InMetadata, TMap<FString, FString>& OutFinalData, rail::RailArray<rail::RailKeyValue>& OutRailKeyValuePairs);
	void OnRailEventFriendsSetMetadataResult(const rail::rail_event::RailFriendsSetMetadataResult* MetadataResult);

	/** Input metadata information */
	FMetadataPropertiesRail Metadata;
	/** Output presence result */
	FSetUserMetadataTaskResult TaskResult;
	/** Delegate to fire on completion */
	FOnOnlineAsyncTaskRailSetUserMetadataComplete CompletionDelegate;
};

/**
 * Builds on FOnlineAsyncTaskRailSetUserMetadata to provide the ability to dynamically store multiple keys
 * by storing an additional key with a list of the others
 */
class FOnlineAsyncTaskRailSetUserPresence
	: public FOnlineAsyncTaskRailSetUserMetadata
{

public:

	FOnlineAsyncTaskRailSetUserPresence() = delete;
	FOnlineAsyncTaskRailSetUserPresence(FOnlineSubsystemTencent* InSubsystem, const FMetadataPropertiesRail& InMetdata, const FOnOnlineAsyncTaskRailSetUserMetadataComplete& InCompletionDelegate)
		: FOnlineAsyncTaskRailSetUserMetadata(InSubsystem, InMetdata, InCompletionDelegate)
	{
	}

	virtual ~FOnlineAsyncTaskRailSetUserPresence()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailSetUserPresence bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

protected:

	//~ Begin FOnlineAsyncTaskRailSetUserMetadata interface
	virtual bool GenerateRailMetadata(FMetadataPropertiesRail& InMetadata, TMap<FString, FString>& OutFinalData, rail::RailArray<rail::RailKeyValue>& OutRailKeyValuePairs) override;
	//~ End FOnlineAsyncTaskRailSetUserMetadata interface
};

class FOnlineAsyncTaskRailSetSessionMetadata
	: public FOnlineAsyncTaskRailSetUserMetadata
{

public:

	FOnlineAsyncTaskRailSetSessionMetadata() = delete;
	FOnlineAsyncTaskRailSetSessionMetadata(FOnlineSubsystemTencent* InSubsystem, const FMetadataPropertiesRail& InMetdata, const FOnOnlineAsyncTaskRailSetUserMetadataComplete& InCompletionDelegate)
		: FOnlineAsyncTaskRailSetUserMetadata(InSubsystem, InMetdata, InCompletionDelegate)
	{
	}

	virtual ~FOnlineAsyncTaskRailSetSessionMetadata()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailSetSessionMetadata bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface
};

DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailSetInviteCommandlineComplete, const FSetUserMetadataTaskResult& /*Result*/);

class FOnlineAsyncTaskRailSetInviteCommandline
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailSetInviteCommandline() = delete;
	FOnlineAsyncTaskRailSetInviteCommandline(FOnlineSubsystemTencent* InSubsystem, const FString& InCmdline, const FOnOnlineAsyncTaskRailSetInviteCommandlineComplete& InCompletionDelegate)
		: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventFriendsSetMetadataResult})
		, Cmdline(InCmdline)
		, CompletionDelegate(InCompletionDelegate)
	{
	}

	virtual ~FOnlineAsyncTaskRailSetInviteCommandline()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailSetCommandline bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:

	void OnRailEventFriendsSetMetadataResult(const rail::rail_event::RailFriendsSetMetadataResult* MetadataResult);

	/** Command line to store with the backend */
	FString Cmdline;
	/** Output result of this call */
	FSetUserMetadataTaskResult TaskResult;
	/** Delegate to fire on completion */
	FOnOnlineAsyncTaskRailSetInviteCommandlineComplete CompletionDelegate;
};

/**
 * Result structure after getting user metadata / presence
 */
struct FGetUserMetadataTaskResult
{
	/** Error information */
	FOnlineError Error;
	/** User whose presence was retrieved */
	FUniqueNetIdRailPtr UserId;
	/** Retrieved presence information */
	FMetadataPropertiesRail Metadata;
};

/**
 * Delegate fired when FOnlineAsyncTaskRailGetUserMetadata completes
 *
 * @param Result the result of the call
 */
DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailGetUserMetadataComplete, const FGetUserMetadataTaskResult& /*Result*/);

class FOnlineAsyncTaskRailGetUserMetadata
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailGetUserMetadata() = delete;
	FOnlineAsyncTaskRailGetUserMetadata(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const TArray<FString>& InMetadataKeys, const FOnOnlineAsyncTaskRailGetUserMetadataComplete& InCompletionDelegate);
	virtual ~FOnlineAsyncTaskRailGetUserMetadata();

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override;
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:

	/**
	 * Retrieve a list of metadata key value pairs
	 *
	 * @param InMetadata list of keys to retrieve from the user specified by this task
	 */
	rail::RailResult QueryMetadata(const TArray<FString>& InMetadata);
	virtual void OnRailEventFriendsGetMetadataResult(const rail::rail_event::RailFriendsGetMetadataResult* MetadataResult);

	/** User to query presence for */
	rail::RailID RailUserId;
	/** Presence retrieved for this user */
	TArray<FString> MetadataKeys;
	/** Output result from the task */
	FGetUserMetadataTaskResult TaskResult;
	/** Delegate to fire when this task is completed */
	FOnOnlineAsyncTaskRailGetUserMetadataComplete CompletionDelegate;
};

/**
 * Builds on FOnlineAsyncTaskRailGetUserMetadata to provide the ability to dynamically retrieve multiple keys
 * from an additional key with a list of the others
 */
class FOnlineAsyncTaskRailGetUserPresence
	: public FOnlineAsyncTaskRailGetUserMetadata
{

public:

	FOnlineAsyncTaskRailGetUserPresence() = delete;
	FOnlineAsyncTaskRailGetUserPresence(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const FOnOnlineAsyncTaskRailGetUserMetadataComplete& InCompletionDelegate);
	virtual ~FOnlineAsyncTaskRailGetUserPresence();

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual FString ToString() const override;
	//~ End FOnlineAsyncTask interface

protected:

	/** Has the list of keys been queried */
	bool bKeysQueried;
	virtual void OnRailEventFriendsGetMetadataResult(const rail::rail_event::RailFriendsGetMetadataResult* MetadataResult) override;
};

/**
 * Result structure after getting a user command line invite
 */
struct FGetInviteCommandLineTaskResult
{
	/** Error information */
	FOnlineError Error;
	/** User whose command line invite was retrieved */
	FUniqueNetIdRailPtr UserId;
	/** Retrieved command line */
	FString Commandline;
};

DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailGetInviteCommandLineComplete, const FGetInviteCommandLineTaskResult& /*Result*/);

class FOnlineAsyncTaskRailGetInviteCommandline
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailGetInviteCommandline() = delete;
	FOnlineAsyncTaskRailGetInviteCommandline(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const FOnOnlineAsyncTaskRailGetInviteCommandLineComplete& InCompletionDelegate);
	virtual ~FOnlineAsyncTaskRailGetInviteCommandline();

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailGetInviteCommandline bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:

	void OnRailEventFriendsGetInviteCommandLine(const rail::rail_event::RailFriendsGetInviteCommandLine* InviteResult);

	/** User to query presence for */
	rail::RailID RailUserId;
	/** Output result from the task */
	FGetInviteCommandLineTaskResult TaskResult;
	/** Delegate to fire when this task is completed */
	FOnOnlineAsyncTaskRailGetInviteCommandLineComplete CompletionDelegate;
};

/**
 * Result structure after getting a complete user invite
 */
struct FGetUserInviteTaskResult
{
	/** Error information */
	FOnlineError Error;
	/** User whose invite was retrieved */
	FUniqueNetIdRailPtr UserId;
	/** Retrieved presence information */
	FMetadataPropertiesRail Metadata;
	/** Retrieved presence information */
	FString Commandline;
};

DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailGetUserInviteComplete, const FGetUserInviteTaskResult& /*Result*/);

class FOnlineAsyncTaskRailGetUserInvite
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailGetUserInvite() = delete;
	FOnlineAsyncTaskRailGetUserInvite(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const FOnOnlineAsyncTaskRailGetUserInviteComplete& InCompletionDelegate);
	virtual ~FOnlineAsyncTaskRailGetUserInvite();

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailGetUserInvite Meta: %d CmdLine: %d"), static_cast<int32>(MetadataReceivedResult), static_cast<int32>(CommandLineResult)); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:

	void OnRailEventFriendsGetMetadataResult(const rail::rail_event::RailFriendsGetMetadataResult* MetadataResult);
	void OnRailEventFriendsGetInviteCommandLine(const rail::rail_event::RailFriendsGetInviteCommandLine* InviteResult);
	/** Called when each event completes to check for total completion */
	void OnEventComplete();

	/** 
	 * Get the session invite keys out of the command line 
	 *
	 * @param InviteCommandLine string containing a list of session invite related keys to retrieve from a remote user
	 */
	void RetrieveFromInviteCommandLine(const FString& InviteCommandline);

	/** Has the metadata callback fired */
	bool bMetadataReceived;
	/** Result of the metadata callback */
	rail::RailResult MetadataReceivedResult;
	/** Has the command line callback fired */
	bool bCommandLineReceived;
	/** Result of the command line callback */
	rail::RailResult CommandLineResult;

	/** User to query invite from (Inviter) */
	rail::RailID RailUserId;
	/** Presence retrieved for this user used to build session search result */
	TArray<FString> MetadataKeys;
	/** Output result from the task */
	FGetUserInviteTaskResult TaskResult;
	/** Delegate to fire when this task is completed */
	FOnOnlineAsyncTaskRailGetUserInviteComplete CompletionDelegate;
};

class FOnlineAsyncTaskRailClearAllMetadata
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailClearAllMetadata() = delete;
	FOnlineAsyncTaskRailClearAllMetadata(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId);
	virtual ~FOnlineAsyncTaskRailClearAllMetadata();

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailClearAllMetadata bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:

	void OnRailEventFriendsClearAllMetadataResult(const rail::rail_event::RailFriendsClearMetadataResult* MetadataResult);

	/** User to clear metadata for */
	rail::RailID RailUserId;
	FOnlineError TaskResult;
};

/**
 * Delegate fired when FOnlineAsyncTaskRailAddFriend completes
 *
 * @param Result the result of the call
 */
DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailAddFriendComplete, const FOnlineError& /*Result*/);

class FOnlineAsyncTaskRailAddFriend
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailAddFriend() = delete;
	/**
	 * Constructor
	 * @param InSubsystem the owning subsystem
	 * @param InUserId the rail id of the player you want to friend
	 * @param InCompletionDelegate the delegate to trigger when this request has completed
	 */
	FOnlineAsyncTaskRailAddFriend(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const FOnOnlineAsyncTaskRailAddFriendComplete& InCompletionDelegate);
	virtual ~FOnlineAsyncTaskRailAddFriend();

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailAddFriend bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:
	
	void OnRailEventFriendsAddFriendResult(const rail::rail_event::RailFriendsAddFriendResult* AddFriendResult);

	/** User to friend */
	rail::RailID RailUserId;
	/** The result of the add friend request */
	FOnlineError TaskResult;
	/** Delegate to trigger on completion */
	FOnOnlineAsyncTaskRailAddFriendComplete CompletionDelegate;
};

/**
 * Result structure after getting users info
 */
struct FGetUsersInfoTaskResult
{
	/** Error information */
	FOnlineError Error;
	/** Users successfully queried */
	TArray<TSharedRef<FOnlineUserInfoTencent>> UserInfos;
};

/**
 * Delegate fired when FOnlineAsyncTaskRailGetUsersInfo completes
 *
 * @param Result the result of the call
 */
DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailGetUsersInfoComplete, const FGetUsersInfoTaskResult& /*Result*/);

class FOnlineAsyncTaskRailGetUsersInfo
	: public FOnlineAsyncTaskRail
{
public:

	FOnlineAsyncTaskRailGetUsersInfo() = delete;
	FOnlineAsyncTaskRailGetUsersInfo(FOnlineSubsystemTencent* InSubsystem, const TArray<FUniqueNetIdRef>& InUserIds, const FOnOnlineAsyncTaskRailGetUsersInfoComplete& InCompletionDelegate);
	virtual ~FOnlineAsyncTaskRailGetUsersInfo() = default;

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailGetUsersInfo bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:
	void OnRailEventUsersGetUsersInfo(const rail::rail_event::RailUsersInfoData* UsersInfoData);

	/** Request id counter so we can map rail responses to the correct requests */
	static FThreadSafeCounter RequestIdCounter;
	/** Our request id */
	int32 RequestId;
	/** User to get data for */
	TArray<FUniqueNetIdRef> UserIds;
	/** Result of the task */
	FGetUsersInfoTaskResult TaskResult;
	/** Delegate to fire on completion */
	FOnOnlineAsyncTaskRailGetUsersInfoComplete CompletionDelegate;
};

class FOnlineAsyncEventRailSystemStateChanged
	: public FOnlineAsyncEventRail
{

public:

	FOnlineAsyncEventRailSystemStateChanged() = delete;
	FOnlineAsyncEventRailSystemStateChanged(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::RailSystemStateChanged* InParams)
		: FOnlineAsyncEventRail(InSubsystem)
		, State(InParams ? InParams->state : rail::RailSystemState::kSystemStateUnknown)
	{
	}

	virtual ~FOnlineAsyncEventRailSystemStateChanged() = default;

	//~ Begin FOnlineAsyncTask interface
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventRailSystemStateChanged State: %s"), *LexToString(State));
	}
	//~ End FOnlineAsyncTask interface

protected:

	rail::RailSystemState State;
};

class FOnlineAsyncEventRailShowFloatingWindow
	: public FOnlineAsyncEventRail
{

public:

	FOnlineAsyncEventRailShowFloatingWindow() = delete;
	FOnlineAsyncEventRailShowFloatingWindow(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::ShowFloatingWindowResult* InParams);

	virtual ~FOnlineAsyncEventRailShowFloatingWindow()
	{
	}

	//~ Begin FOnlineAsyncEvent interface
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncEventRailShowFloatingWindow Result: %s bWasOpened: %d"), *TaskResult.Error.ToLogString(), TaskResult.bOpened); }
	//~ End FOnlineAsyncEvent interface

protected:

	/** Result of the task */
	FFloatingWindowTaskResult TaskResult;
};

struct FOnlineAsyncEventRailShowFloatingNotifyWindowPayload
	: public FJsonSerializable
{
public:
	/** Title of dialog to display*/
	FString DialogTitle;
	/** Text of dialog to display */
	FString DialogText;
	/** Text to display in OK button */
	FString ButtonText;

public:
	BEGIN_JSON_SERIALIZER
	JSON_SERIALIZE("title", DialogTitle);
	JSON_SERIALIZE("desc", DialogText);
	JSON_SERIALIZE("confirm_button", ButtonText);
	END_JSON_SERIALIZER
};

class FOnlineAsyncEventRailShowFloatingNotifyWindow
	: public FOnlineAsyncEventRail
{

public:

	FOnlineAsyncEventRailShowFloatingNotifyWindow() = delete;
	FOnlineAsyncEventRailShowFloatingNotifyWindow(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::ShowNotifyWindow* InParams);

	virtual ~FOnlineAsyncEventRailShowFloatingNotifyWindow()
	{
	}

	//~ Begin FOnlineAsyncEvent interface
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncEventRailShowFloatingNotifyWindow")); }
	//~ End FOnlineAsyncEvent interface
	
	bool bShowAntiAddictionMessage;
	FOnlineAsyncEventRailShowFloatingNotifyWindowPayload Payload;

};

class FOnlineAsyncEventRailInviteSent
	: public FOnlineAsyncEventRail
{

public:

	FOnlineAsyncEventRailInviteSent() = delete;
	FOnlineAsyncEventRailInviteSent(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::RailUsersNotifyInviter* InParams)
		: FOnlineAsyncEventRail(InSubsystem)
		, Result(InParams ? InParams->get_result() : rail::kErrorUnknown)
		, InviterId(InParams ? InParams->rail_id : rail::kInvalidRailId)
		, InviteeId(InParams ? InParams->invitee_id : rail::kInvalidRailId)
		, UserData(InParams ? InParams->user_data : rail::RailString())
	{
	}

	virtual ~FOnlineAsyncEventRailInviteSent()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncEventRailInviteSent Inviter: %llu Invitee: %llu bWasSuccessful: %d"), InviterId.get_id(), InviteeId.get_id(), (Result == rail::kSuccess)); }
	//~ End FOnlineAsyncTask interface

protected:

	/** Result of the invite sent event */
	rail::RailResult Result;
	/** User id who sent the invite */
	rail::RailID InviterId;
	/** User id who will receive the invite */
	rail::RailID InviteeId;
	/** User data returned with the call */
	rail::RailString UserData;
};

class FOnlineAsyncEventRailInviteSentEx
	: public FOnlineAsyncEventRail
{

public:

	FOnlineAsyncEventRailInviteSentEx() = delete;
	FOnlineAsyncEventRailInviteSentEx(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::RailUsersInviteUsersResult* InParams)
		: FOnlineAsyncEventRail(InSubsystem)
		, Result(InParams ? InParams->get_result() : rail::kErrorUnknown)
		, InviteType(InParams ? InParams->invite_type : rail::kRailUsersInviteTypeGame)
		, UserData(InParams ? InParams->user_data : rail::RailString())
	{
	}

	virtual ~FOnlineAsyncEventRailInviteSentEx()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncEventRailInviteSentEx bWasSuccessful: %d"), (Result == rail::kSuccess)); }
	//~ End FOnlineAsyncTask interface

protected:

	/** Result of the invite sent event */
	rail::RailResult Result;
	/** Type of invite */
	rail::EnumRailUsersInviteType InviteType;
	/** User data returned with the call */
	rail::RailString UserData;
};

class FOnlineAsyncEventRailInviteResponse
	: public FOnlineAsyncEventRail
{

public:

	FOnlineAsyncEventRailInviteResponse() = delete;
	FOnlineAsyncEventRailInviteResponse(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::RailUsersRespondInvitation* InParams)
		: FOnlineAsyncEventRail(InSubsystem)
		, Result(InParams ? InParams->get_result() : rail::kErrorUnknown)
		, InviterId(InParams ? InParams->inviter_id : rail::kInvalidRailId)
		, InviteeId(InParams ? InParams->rail_id : rail::kInvalidRailId)
		, UserData(InParams ? InParams->user_data : rail::RailString())
		, Response(InParams ? InParams->response : rail::kRailInviteResponseTypeUnknown)
	{
	}

	virtual ~FOnlineAsyncEventRailInviteResponse()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncEventRailInviteResponse Inviter: %llu Invitee: %llu bWasSuccessful: %d"), InviterId.get_id(), InviteeId.get_id(), (Result == rail::kSuccess)); }
	//~ End FOnlineAsyncTask interface

protected:

	/** Result of the invite response event */
	rail::RailResult Result;
	/** User id who sent the invite */
	rail::RailID InviterId;
	/** User id who received the invite */
	rail::RailID InviteeId;
	/** User data returned with the call */
	rail::RailString UserData;
	/** Invitees response to the invite */
	rail::EnumRailUsersInviteResponseType Response;
};

class FOnlineAsyncEventRailJoinGameByUser
	: public FOnlineAsyncEventRail
{

public:

	FOnlineAsyncEventRailJoinGameByUser() = delete;
	FOnlineAsyncEventRailJoinGameByUser(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::RailPlatformNotifyEventJoinGameByUser* InParams)
		: FOnlineAsyncEventRail(InSubsystem)
		, Result(InParams ? InParams->get_result() : rail::kErrorUnknown)
		, UserToJoin(InParams ? InParams->rail_id_to_join : rail::kInvalidRailId)
		, CommandLine(InParams ? InParams->commandline_info : rail::RailString())
	{
	}

	virtual ~FOnlineAsyncEventRailJoinGameByUser()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncEventRailJoinGameByUser RemoteUser: %llu Commandline: %s bWasSuccessful: %d"), UserToJoin.get_id(), ANSI_TO_TCHAR(CommandLine.c_str()), (Result == rail::kSuccess)); }
	//~ End FOnlineAsyncTask interface

protected:

	/** Result of the invite response event */
	rail::RailResult Result;
	/** Remote user to join */
	rail::RailID UserToJoin;
	/** Commandline invite of remote user */
	rail::RailString CommandLine;
};

class FOnlineAsyncEventRailJoinGameResult
	: public FOnlineAsyncEventRail
{

public:

	FOnlineAsyncEventRailJoinGameResult() = delete;
	FOnlineAsyncEventRailJoinGameResult(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::RailUsersInviteJoinGameResult* InParams)
		: FOnlineAsyncEventRail(InSubsystem)
		, Result(InParams ? InParams->get_result() : rail::kErrorUnknown)
		, InviterId(InParams ? InParams->rail_id : rail::kInvalidRailId)
		, InviteeId(InParams ? InParams->invitee_id : rail::kInvalidRailId)
		, UserData(InParams ? InParams->user_data : rail::RailString())
		, Response(InParams ? InParams->response_value : rail::kRailInviteResponseTypeUnknown)
		, InviteType(InParams ? InParams->invite_type : rail::kRailUsersInviteTypeGame)
	{
	}

	virtual ~FOnlineAsyncEventRailJoinGameResult()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncEventRailJoinGameResult Inviter: %llu Invitee: %llu bWasSuccessful: %d"), InviterId.get_id(), InviteeId.get_id(), (Result == rail::kSuccess)); }
	//~ End FOnlineAsyncTask interface

protected:

	/** Result of the join game event */
	rail::RailResult Result;
	/** User id who sent the invite */
	rail::RailID InviterId;
	/** User id who received the invite */
	rail::RailID InviteeId;
	/** User data returned with the call */
	rail::RailString UserData;
	/** Invitees response to the invite */
	rail::EnumRailUsersInviteResponseType Response;
	/** Type of invite sent */
	rail::EnumRailUsersInviteType InviteType;
};

class FOnlineAsyncEventRailFriendsListChanged
	: public FOnlineAsyncEventRail
{
public:

	FOnlineAsyncEventRailFriendsListChanged() = delete;
	FOnlineAsyncEventRailFriendsListChanged(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::RailFriendsListChanged* InParams)
		: FOnlineAsyncEventRail(InSubsystem)
		, UserId(InParams->rail_id)
	{
	}

	virtual ~FOnlineAsyncEventRailFriendsListChanged() = default;

	//~ Begin FOnlineAsyncTask interface
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventRailFriendsListChanged"));
	}
	//~ End FOnlineAsyncTask interface

private:
	/** The user id whose list has changed */
	rail::RailID UserId;
};

class FOnlineAsyncEventRailFriendsOnlineStateChanged
	: public FOnlineAsyncEventRail
{
public:

	FOnlineAsyncEventRailFriendsOnlineStateChanged() = delete;
	FOnlineAsyncEventRailFriendsOnlineStateChanged(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::RailFriendsOnlineStateChanged* InParams)
		: FOnlineAsyncEventRail(InSubsystem)
		, OnlineState(InParams ? InParams->friend_online_state : rail::RailFriendOnLineState())
	{
	}

	virtual ~FOnlineAsyncEventRailFriendsOnlineStateChanged() = default;

	//~ Begin FOnlineAsyncTask interface
	virtual void Finalize() override;
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventRailFriendsOnlineStateChanged friend_rail_id=%llu, friend_online_state=%s, game_define_game_playing_state=%u"),
			OnlineState.friend_rail_id.get_id(), *LexToString(OnlineState.friend_online_state), OnlineState.game_define_game_playing_state);
	}
	//~ End FOnlineAsyncTask interface

private:
	/** Data related to the online state change */
	rail::RailFriendOnLineState OnlineState;
};

class FOnlineAsyncEventRailFriendsMetadataChanged
	: public FOnlineAsyncEventRail
{
public:

	FOnlineAsyncEventRailFriendsMetadataChanged() = delete;
	FOnlineAsyncEventRailFriendsMetadataChanged(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::RailFriendsMetadataChanged* InParams);

	virtual ~FOnlineAsyncEventRailFriendsMetadataChanged() = default;

	//~ Begin FOnlineAsyncTask interface
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventRailFriendsMetadataChanged size: %d"), ChangedMetadata.Num());
	}
	//~ End FOnlineAsyncTask interface

private:
	/** Data related to the metadata change */
	TUniqueNetIdMap<FMetadataPropertiesRail> ChangedMetadata;
};

/**
 * Result structure after getting users presence
 */
struct FQueryUserPresenceTaskResult
{
	FQueryUserPresenceTaskResult()
		: QueryCount(0)
		, SuccessCount(0)
	{}

	/** Error information */
	FOnlineError Error;
	/** Number of completed queries */
	int32 QueryCount;
	/** Number of successful queries */
	int32 SuccessCount;
};

/**
 * Query a list of friends presence to prime the presence cache
 */
class FOnlineAsyncTaskRailQueryFriendsPresence
	: public FOnlineAsyncTaskBasic<FOnlineSubsystemTencent>
{

public:

	/** Delegate fired when this task completed */
	DECLARE_DELEGATE_OneParam(FCompletionDelegate, const FQueryUserPresenceTaskResult& /*TaskResult*/);

	FOnlineAsyncTaskRailQueryFriendsPresence() = delete;
	FOnlineAsyncTaskRailQueryFriendsPresence(FOnlineSubsystemTencent* InSubsystem, TArray<FUniqueNetIdRef>& InFriends, const FCompletionDelegate& InCompletionDelegate)
		: FOnlineAsyncTaskBasic(InSubsystem)
		, CompletionDelegate(InCompletionDelegate)
		, FriendsList(InFriends)	
	{
	}

	virtual ~FOnlineAsyncTaskRailQueryFriendsPresence()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailQueryFriendsPresence bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

protected:

	/** Completion delegate to fire */
	FCompletionDelegate CompletionDelegate;
	/** List of friends whose presence data is requested */
	TArray<FUniqueNetIdRef> FriendsList;
	/** Result of operation */
	FQueryUserPresenceTaskResult TaskResult;
};

/**
 * Result structure after sending an invite
 */
struct FSendInviteTaskResult
{
	FSendInviteTaskResult() = delete;
	FSendInviteTaskResult(const FString& InInviteStr)
		: InviteStr(InInviteStr)
	{ }

	/** Invite command line */
	FString InviteStr;
	/** Error information */
	FOnlineError Error;
};

/**
 * Delegate fired when FOnlineAsyncTaskRailSendInvite completes
 *
 * @param Result the result of the call
 */
DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailSendInviteComplete, const FSendInviteTaskResult& /*Result*/);

/**
 * Send an invite to a list of friends
 * Contains a string with the various metadata keys to query relating to the invite
 */
class FOnlineAsyncTaskRailSendInvite
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailSendInvite() = delete;
	FOnlineAsyncTaskRailSendInvite(FOnlineSubsystemTencent* InSubsystem, const FString& InInviteStr, const TArray<FUniqueNetIdRef>& InUserIds, const FOnOnlineAsyncTaskRailSendInviteComplete& InCompletionDelegate)
		: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventUsersInviteUsersResult})
		, UserIds(InUserIds)
		, CompletionDelegate(InCompletionDelegate)
		, TaskResult(InInviteStr)
	{
	}

	virtual ~FOnlineAsyncTaskRailSendInvite()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailSendInvite bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:

	void OnRailEventFriendsSendInviteResult(const rail::rail_event::RailUsersInviteUsersResult* InviteResult);

	/** List of users to send invite to */
	TArray<FUniqueNetIdRef> UserIds;
	/** Delegate to fire on completion */
	FOnOnlineAsyncTaskRailSendInviteComplete CompletionDelegate;
	/** Task Result */
	FSendInviteTaskResult TaskResult;
};

/**
 * Result structure after retrieving details about an invite
 */
struct FGetInviteDetailsTaskResult
{
	/** Error information */
	FOnlineError Error;
	/** Invite Details */
	FString InviteString;
};

/**
 * Delegate fired when FOnlineAsyncTaskRailGetInviteDetails completes
 *
 * @param Result the result of the call
 */
DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailGetInviteDetailsComplete, const FGetInviteDetailsTaskResult& /*Result*/);

/**
 * Get the details related to an invite sent by a remote user
 */
class FOnlineAsyncTaskRailGetInviteDetails
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailGetInviteDetails() = delete;
	FOnlineAsyncTaskRailGetInviteDetails(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const FOnOnlineAsyncTaskRailGetInviteDetailsComplete& InCompletionDelegate);

	virtual ~FOnlineAsyncTaskRailGetInviteDetails()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailGetInviteDetails bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:

	void OnRailEventFriendsGetInviteDetailsResult(const rail::rail_event::RailUsersGetInviteDetailResult* InviteDetails);

	/** User to query presence for */
	rail::RailID RailUserId;
	/** Delegate to fire on completion */
	FOnOnlineAsyncTaskRailGetInviteDetailsComplete CompletionDelegate;
	/** Task Result */
	FGetInviteDetailsTaskResult TaskResult;
};

/**
 * Result structure containing details about friends playing games on the platform
 */
struct FQueryFriendPlayedGamesTaskResult
{
	/** Error information */
	FOnlineError Error;
	/** Friend queried */
	FUniqueNetIdRailPtr FriendId;

	struct FRailGamePlayedInfo
	{
		/** Are we in a Rail Server (not implemented) */
		bool bInServer;
		/** Are we in a Rail Room (not implemented) */
		bool bInRoom;
		///** Server Id */
		//uint64 ServerId;
		///** Room Id */
		//uint64 RoomId;
		/** Id of the game being played */
		rail::RailGameID GameId;
		/** Game state of the user */
		rail::RailFriendPlayedGamePlayState GamePlayState;
	};

	/** Array of games being played on the platform */
	TArray<FRailGamePlayedInfo> GameInfos;
};

/**
 * Delegate fired when FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo completes
 *
 * @param Result the result of the call
 */
DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailQueryFriendPlayedGamesComplete, const FQueryFriendPlayedGamesTaskResult& /*Result*/);

/**
 * Get details about a friend playing games on the RailSDK
 * May contain an array of games information if the user has more than one open at the same time
 */
class FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo() = delete;
	FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRail& InUserId, const FOnOnlineAsyncTaskRailQueryFriendPlayedGamesComplete& InCompletionDelegate);

	virtual ~FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:

	void OnRailEventQueryFriendPlayedGamesResult(const rail::rail_event::RailFriendsQueryFriendPlayedGamesResult* PlayedGamesDetail);

	/** Delegate to fire on completion */
	FOnOnlineAsyncTaskRailQueryFriendPlayedGamesComplete CompletionDelegate;
	/** Task Result */
	FQueryFriendPlayedGamesTaskResult TaskResult;
};

/**
 * Result structure containing details reporting players that were played with
 */
struct FReportPlayedWithUsersTaskResult
{
	/** Error information */
	FOnlineError Error;
	/** Users reported */
	TArray<FReportPlayedWithUser> UsersReported;
};

/**
 * Delegate fired when FOnlineAsyncTaskRailReportPlayedWithUsers completes
 *
 * @param Result the result of the call
 */
DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailReportPlayedWithUsersComplete, const FReportPlayedWithUsersTaskResult& /*Result*/);

/**
 * Report details about a players that have been seen over the course of a game
 */
class FOnlineAsyncTaskRailReportPlayedWithUsers
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailReportPlayedWithUsers() = delete;
	FOnlineAsyncTaskRailReportPlayedWithUsers(FOnlineSubsystemTencent* InSubsystem, const TArray<FReportPlayedWithUser>& InUsersReported, const FOnOnlineAsyncTaskRailReportPlayedWithUsersComplete& InCompletionDelegate);

	virtual ~FOnlineAsyncTaskRailReportPlayedWithUsers()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailReportPlayedWithUsers bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:

	void OnRailEventReportPlayedWithUsers(const rail::rail_event::RailFriendsReportPlayedWithUserListResult* PlayedGamesDetail);

	/** Delegate to fire on completion */
	FOnOnlineAsyncTaskRailReportPlayedWithUsersComplete CompletionDelegate;
	/** Task Result */
	FReportPlayedWithUsersTaskResult TaskResult;
};

/**
 * Result structure containing details of users last played with (not just friends)
 */
struct FQueryPlayedWithFriendsListTaskResult
{
	/** Error information */
	FOnlineError Error;
	/** Users reported */
	TArray<FUniqueNetIdRef> UsersPlayedWith;
};

/**
 * Delegate fired when FOnlineAsyncTaskRailQueryPlayedWithFriendsList completes
 *
 * @param Result the result of the call
 */
DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailQueryPlayedWithFriendsListComplete, const FQueryPlayedWithFriendsListTaskResult& /*Result*/);

/**
 * Report details about users the local user has played with
 */
class FOnlineAsyncTaskRailQueryPlayedWithFriendsList
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailQueryPlayedWithFriendsList() = delete;
	FOnlineAsyncTaskRailQueryPlayedWithFriendsList(FOnlineSubsystemTencent* InSubsystem, const FOnOnlineAsyncTaskRailQueryPlayedWithFriendsListComplete& InCompletionDelegate);

	virtual ~FOnlineAsyncTaskRailQueryPlayedWithFriendsList()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailQueryPlayedWithFriendsList bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:

	void OnRailEventPlayedWithFriendsList(const rail::rail_event::RailFriendsQueryPlayedWithFriendsListResult* PlayedFriendsList);

	/** Delegate to fire on completion */
	FOnOnlineAsyncTaskRailQueryPlayedWithFriendsListComplete CompletionDelegate;
	/** Task Result */
	FQueryPlayedWithFriendsListTaskResult TaskResult;
};

/**
 * Result structure containing details of the time a user was last seen
 */
struct FQueryPlayedWithFriendsTimeTaskResult
{
	/** Error information */
	FOnlineError Error;
	/** List of users to query */
	TArray<FUniqueNetIdRef> UserIds;
	/** Last seen info for a single player */
	struct FRecentPlayers 
	{
		FRecentPlayers() = delete;
		FRecentPlayers(const FUniqueNetIdRef InUserId, const FDateTime& InLastPlayed)
			: UserId(InUserId)
			, LastPlayed(InLastPlayed)
		{

		}
		FUniqueNetIdRef UserId;
		FDateTime LastPlayed;
	};
	/** All users reported as being seen by this user */
	TArray<FRecentPlayers> LastPlayedWithUsers;
};

/**
 * Delegate fired when FOnlineAsyncTaskRailQueryPlayedWithFriendsList completes
 *
 * @param Result the result of the call
 */
DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailQueryPlayedWithFriendsTimeComplete, const FQueryPlayedWithFriendsTimeTaskResult& /*Result*/);

/**
 * Report details about users the local user has played with
 */
class FOnlineAsyncTaskRailQueryPlayedWithFriendsTime
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailQueryPlayedWithFriendsTime() = delete;
	FOnlineAsyncTaskRailQueryPlayedWithFriendsTime(FOnlineSubsystemTencent* InSubsystem, const TArray<FUniqueNetIdRef>& InUserIds, const FOnOnlineAsyncTaskRailQueryPlayedWithFriendsTimeComplete& InCompletionDelegate);

	virtual ~FOnlineAsyncTaskRailQueryPlayedWithFriendsTime()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void Finalize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailQueryPlayedWithFriendsTime bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID event_id, rail::EventBase* param) override;
	//~ End IRailEvent interface

protected:

	void OnRailEventPlayedWithFriendsTime(const rail::rail_event::RailFriendsQueryPlayedWithFriendsTimeResult* PlayedTimeResult);

	/** Delegate to fire on completion */
	FOnOnlineAsyncTaskRailQueryPlayedWithFriendsTimeComplete CompletionDelegate;
	/** Task Result */
	FQueryPlayedWithFriendsTimeTaskResult TaskResult;
};

/**
 * Result structure for requesting all receipts
 */
struct FRequestAllAssetsTaskResult
{
	/** Error information */
	FOnlineError Error;
	/** Purchase receipts */
	FPurchaseReceipt PurchaseReceipt;
};

/**
 * Delegate fired when FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo completes
 *
 * @param Result the result of the call
 */
DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailRequestAllAssetsComplete, const FRequestAllAssetsTaskResult& /*Result*/);

class FOnlineAsyncTaskRailRequestAllAssets
	: public FOnlineAsyncTaskRail
{
public:
	FOnlineAsyncTaskRailRequestAllAssets() = delete;
	FOnlineAsyncTaskRailRequestAllAssets(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRef& InUserId, const FOnOnlineAsyncTaskRailRequestAllAssetsComplete& InCompletionDelegate)
		: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventAssetsRequestAllAssetsFinished})
		, UserId(InUserId)
		, CompletionDelegate(InCompletionDelegate)
	{
	}

	virtual ~FOnlineAsyncTaskRailRequestAllAssets()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailRequestAllAssets bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID EventId, rail::EventBase* Param) override;
	//~ End IRailEvent interface
protected:

	void OnRailRequestAllAssetsFinished(const rail::rail_event::RequestAllAssetsFinished* AssetsResponse);

	/** User we are requesting assets for */
	const FUniqueNetIdRef UserId;
	/** Delegate to fire on completion */
	FOnOnlineAsyncTaskRailRequestAllAssetsComplete CompletionDelegate;
	/** Task Result */
	FRequestAllAssetsTaskResult TaskResult;
};

class FOnlineAsyncEventRailAssetsChanged
	: public FOnlineAsyncEventRail
{
public:

	FOnlineAsyncEventRailAssetsChanged() = delete;
	FOnlineAsyncEventRailAssetsChanged(FOnlineSubsystemTencent* InSubsystem, const rail::rail_event::RailAssetsChanged* InParams);

	virtual ~FOnlineAsyncEventRailAssetsChanged() = default;

	//~ Begin FOnlineAsyncTask interface
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("FOnlineAsyncEventRailAssetsChanged"));
	}
	//~ End FOnlineAsyncTask interface

private:
};

/**
 * Result structure for requesting all purchasable products
 */
struct FRequestAllPurchasableProductsTaskResult
{
	/** Error information */
	FOnlineError Error;
	/** Purchasable products */
	TArray<FOnlineStoreOfferRef> Offers;
};

/**
 * Delegate fired when FOnlineAsyncTaskRailRequestAllPurchasableProducts completes
 *
 * @param Result the result of the call
 */
DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailRequestAllPurchasableProductsComplete, const FRequestAllPurchasableProductsTaskResult& /*Result*/);

class FOnlineAsyncTaskRailRequestAllPurchasableProducts
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailRequestAllPurchasableProducts() = delete;
	FOnlineAsyncTaskRailRequestAllPurchasableProducts(FOnlineSubsystemTencent* InSubsystem, const FOnOnlineAsyncTaskRailRequestAllPurchasableProductsComplete& InCompletionDelegate)
		: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventInGamePurchaseAllPurchasableProductsInfoReceived})
		, CompletionDelegate(InCompletionDelegate)
	{
	}

	virtual ~FOnlineAsyncTaskRailRequestAllPurchasableProducts()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailRequestAllPurchasableProducts bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID EventId, rail::EventBase* Param) override;
	//~ End IRailEvent interface

protected:

	void OnRailRequestAllPurchasableProductsResponse(rail::rail_event::RailInGamePurchaseRequestAllPurchasableProductsResponse* Response);

	/** Result of the task */
	FRequestAllPurchasableProductsTaskResult TaskResult;
	/** Delegate to fire on completion */
	FOnOnlineAsyncTaskRailRequestAllPurchasableProductsComplete CompletionDelegate;
};

/**
 * Result structure for purchasing products
 */
struct FPurchaseProductsTaskResult
{
	/** Error information */
	FOnlineError Error;
	/** Purchase receipt, if successful */
	TSharedPtr<FPurchaseReceipt> PurchaseReceipt;
};

/**
 * Delegate fired when FOnlineAsyncTaskRailPurchaseProducts completes
 *
 * @param Result the result of the call
 */
DECLARE_DELEGATE_OneParam(FOnOnlineAsyncTaskRailPurchaseProductsComplete, const FPurchaseProductsTaskResult& /*Result*/);

class FOnlineAsyncTaskRailPurchaseProducts
	: public FOnlineAsyncTaskRail
{

public:

	FOnlineAsyncTaskRailPurchaseProducts() = delete;
	FOnlineAsyncTaskRailPurchaseProducts(FOnlineSubsystemTencent* InSubsystem, const FUniqueNetIdRef& InUserId, const FPurchaseCheckoutRequest& InCheckoutRequest, const FOnOnlineAsyncTaskRailPurchaseProductsComplete& InCompletionDelegate)
		: FOnlineAsyncTaskRail(InSubsystem, {rail::kRailEventInGamePurchasePurchaseProductsToAssetsResult})
		, UserId(InUserId)
		, CheckoutRequest(InCheckoutRequest)
		, CompletionDelegate(InCompletionDelegate)
	{
		// User will be in a dialog, allow the user to take their time.
		bCanTimeout = false;
	}

	virtual ~FOnlineAsyncTaskRailPurchaseProducts()
	{
	}

	//~ Begin FOnlineAsyncTask interface
	virtual void Initialize() override;
	virtual void TriggerDelegates() override;
	virtual FString ToString() const override { return FString::Printf(TEXT("FOnlineAsyncTaskRailPurchaseProducts bWasSuccessful: %d"), WasSuccessful()); }
	//~ End FOnlineAsyncTask interface

	//~ Begin IRailEvent interface
	virtual void OnRailEvent(rail::RAIL_EVENT_ID EventId, rail::EventBase* Param) override;
	//~ End IRailEvent interface

protected:

	void OnRailPurchaseProductsToAssetsResult(rail::rail_event::RailInGamePurchasePurchaseProductsToAssetsResponse* Response);
	void HandlePurchaseProductsResult(const TOptional<rail::RailResult> RailResult);

	/** User performing the purchase */
	const FUniqueNetIdRef UserId;
	/** Items to purchase */
	FPurchaseCheckoutRequest CheckoutRequest;
	/** Result of the task */
	FPurchaseProductsTaskResult TaskResult;
	/** Delegate to fire on completion */
	FOnOnlineAsyncTaskRailPurchaseProductsComplete CompletionDelegate;
};

#endif // WITH_TENCENT_RAIL_SDK
