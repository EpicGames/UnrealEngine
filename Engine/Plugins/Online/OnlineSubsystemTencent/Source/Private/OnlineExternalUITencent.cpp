// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineExternalUITencent.h"

#if WITH_TENCENTSDK
#if WITH_TENCENT_RAIL_SDK

#include "OnlineSubsystemTencent.h"
#include "OnlineAsyncTasksTencent.h"

bool FOnlineExternalUITencent::ShowLoginUI(const int ControllerIndex, bool bShowOnlineOnly, bool bShowSkipButton, const FOnLoginUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUITencent::ShowFriendsUI(int32 LocalUserNum)
{
	FOnlineAsyncTaskRailShowFloatingWindow* NewTask = new FOnlineAsyncTaskRailShowFloatingWindow(TencentSubsystem, rail::EnumRailWindowType::kRailWindowFriendList);
	TencentSubsystem->QueueAsyncTask(NewTask);
	return true;
}

bool FOnlineExternalUITencent::ShowInviteUI(int32 LocalUserNum, FName SessionName)
{
	return false;
}

bool FOnlineExternalUITencent::ShowAchievementsUI(int32 LocalUserNum)
{
	FOnlineAsyncTaskRailShowFloatingWindow* NewTask = new FOnlineAsyncTaskRailShowFloatingWindow(TencentSubsystem, rail::EnumRailWindowType::kRailWindowAchievement);
	TencentSubsystem->QueueAsyncTask(NewTask);
	return true;
}

bool FOnlineExternalUITencent::ShowLeaderboardUI(const FString& LeaderboardName)
{
	FOnlineAsyncTaskRailShowFloatingWindow* NewTask = new FOnlineAsyncTaskRailShowFloatingWindow(TencentSubsystem, rail::EnumRailWindowType::kRailWindowLeaderboard);
	TencentSubsystem->QueueAsyncTask(NewTask);
	return true;
}

bool FOnlineExternalUITencent::ShowWebURL(const FString& Url, const FShowWebUrlParams& ShowParams, const FOnShowWebUrlClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUITencent::CloseWebURL()
{
	return false;
}

bool FOnlineExternalUITencent::ShowProfileUI(const FUniqueNetId& Requestor, const FUniqueNetId& Requestee, const FOnProfileUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUITencent::ShowAccountUpgradeUI(const FUniqueNetId& UniqueId)
{
	return false;
}

bool FOnlineExternalUITencent::ShowStoreUI(int32 LocalUserNum, const FShowStoreParams& ShowParams, const FOnShowStoreUIClosedDelegate& Delegate)
{
	return false;
}

bool FOnlineExternalUITencent::ShowSendMessageUI(int32 LocalUserNum, const FShowSendMessageParams& ShowParams, const FOnShowSendMessageUIClosedDelegate& Delegate)
{
	return false;
}

#endif // WITH_TENCENTSDK
#endif // WITH_TENCENT_RAIL_SDK
