// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineFriendsTencent.h"

#if WITH_TENCENTSDK
#if WITH_TENCENT_RAIL_SDK

#include "Interfaces/OnlineIdentityInterface.h"
#include "Interfaces/OnlineUserInterface.h"
#include "Online/OnlineBase.h"
#include "OnlineIdentityTencent.h"
#include "OnlinePresenceTencent.h"
#include "OnlineSubsystemTencent.h"

FOnlineFriendTencent::FOnlineFriendTencent(FOnlineSubsystemTencent* InTencentSubsystem, const FUniqueNetIdRailRef InUserId)
	: TencentSubsystem(InTencentSubsystem)
	, UserId(InUserId)
{
}

FUniqueNetIdRef FOnlineFriendTencent::GetUserId() const
{
	return UserId;
}

FString FOnlineFriendTencent::GetRealName() const
{
	FString Result;
	GetAccountData(USER_ATTR_REALNAME, Result);
	return Result;
}

FString FOnlineFriendTencent::GetDisplayName(const FString& Platform) const
{
	FString Result;
	GetAccountData(USER_ATTR_DISPLAYNAME, Result);
	return Result;
}

bool FOnlineFriendTencent::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	return GetAccountData(AttrName, OutAttrValue);
}

EInviteStatus::Type FOnlineFriendTencent::GetInviteStatus() const
{
	// We do not have entries for users that we are not friends with
	return EInviteStatus::Accepted;
}

const FOnlineUserPresence& FOnlineFriendTencent::GetPresence() const
{
	TSharedPtr<FOnlineUserPresence> Presence;
	if (TencentSubsystem != nullptr &&
		TencentSubsystem->GetPresenceInterface().IsValid() &&
		TencentSubsystem->GetPresenceInterface()->GetCachedPresence(*UserId, Presence) == EOnlineCachedResult::Success &&
		Presence.IsValid())
	{
		return *Presence;
	}
	else
	{
		static FOnlineUserPresence DefaultPresence;
		return DefaultPresence;
	}
}

FUniqueNetIdRef FOnlineRecentPlayerTencent::GetUserId() const
{
	return UserId;
}

FString FOnlineRecentPlayerTencent::GetRealName() const
{
	FString Result;
	GetAccountData(USER_ATTR_REALNAME, Result);
	return Result;
}

FString FOnlineRecentPlayerTencent::GetDisplayName(const FString& Platform) const
{
	FString Result;
	GetAccountData(USER_ATTR_DISPLAYNAME, Result);
	return Result;
}

bool FOnlineRecentPlayerTencent::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	return GetAccountData(AttrName, OutAttrValue);
}

FDateTime FOnlineRecentPlayerTencent::GetLastSeen() const
{
	return LastSeen;
}

FOnlineFriendsTencent::FOnlineFriendsTencent(FOnlineSubsystemTencent* InSubsystem)
	: TencentSubsystem(InSubsystem)
{
}

FOnlineFriendsTencent::~FOnlineFriendsTencent()
{
	IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		IdentityInt->ClearOnLoginChangedDelegate_Handle(OnLoginChangedHandle);
	}
}

bool FOnlineFriendsTencent::Init()
{
	IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		OnLoginChangedHandle = IdentityInt->AddOnLoginChangedDelegate_Handle(FOnLoginChangedDelegate::CreateThreadSafeSP(this, &FOnlineFriendsTencent::OnLoginChanged));
		return true;
	}

	return false;
}

void FOnlineFriendsTencent::OnLoginChanged(int32 LocalUserNum)
{
	IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		FUniqueNetIdRailPtr UserId = StaticCastSharedPtr<const FUniqueNetIdRail>(IdentityInt->GetUniquePlayerId(LocalUserNum));
		if (UserId.IsValid())
		{
			ELoginStatus::Type LoginStatus = IdentityInt->GetLoginStatus(LocalUserNum);
			if (LoginStatus == ELoginStatus::NotLoggedIn)
			{
				// user logged out so clear any friends/players lists
				FriendsLists.Remove(static_cast<rail::RailID>(*UserId));
				RecentPlayersLists.Remove(UserId.ToSharedRef());
			}
		}
	}
}

bool FOnlineFriendsTencent::ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate /*= FOnReadFriendsListComplete()*/)
{
	FOnlineError Error(EOnlineErrorResult::Unknown);
	bool bIsQueryingUsers = false;

	if (!OnQueryUsersForFriendsListCompleteDelegate.IsValid())
	{
		if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
		{
			IOnlineUserPtr TencentUser = TencentSubsystem->GetUserInterface();
			if (TencentUser.IsValid())
			{
				rail::RailArray<rail::RailFriendInfo> Friends;
				rail::RailResult Result = RailFriends->GetFriendsList(&Friends);
				if (Result == rail::kSuccess)
				{
					FOnlinePresenceTencentPtr PresenceInt = StaticCastSharedPtr<FOnlinePresenceTencent>(TencentSubsystem->GetPresenceInterface());

					TArray<FUniqueNetIdRef> FriendIds;
					for (uint32 RailIdx = 0; RailIdx < Friends.size(); ++RailIdx)
					{
						const rail::RailFriendInfo& RailFriendInfo(Friends[RailIdx]);
						if (RailFriendInfo.friend_rail_id != rail::kInvalidRailId)
						{
							FUniqueNetIdRef FriendId(FUniqueNetIdRail::Create(RailFriendInfo.friend_rail_id));
							UE_LOG_ONLINE_FRIEND(VeryVerbose, TEXT("Friend Id: %s State: %s"), *FriendId->ToDebugString(), *LexToString(RailFriendInfo.online_state.friend_online_state));
							
							PresenceInt->SetUserOnlineState(*FriendId, RailOnlineStateToOnlinePresence(RailFriendInfo.online_state.friend_online_state));
							FriendIds.Add(FriendId);
						}
						else
						{
							UE_LOG_ONLINE_FRIEND(Warning, TEXT("Invalid friend in friends list"));
						}
					}

					Error = FOnlineError::Success();
					if (FriendIds.Num() > 0)
					{
						FOnQueryUserInfoCompleteDelegate CompletionDelegate;
						CompletionDelegate.BindThreadSafeSP(this, &FOnlineFriendsTencent::OnQueryUsersForFriendsListComplete, ListName, FriendIds);

						OnQueryUsersForFriendsListCompleteDelegate = TencentUser->AddOnQueryUserInfoCompleteDelegate_Handle(LocalUserNum, CompletionDelegate);
						UE_LOG_ONLINE_FRIEND(VeryVerbose, TEXT("ReadFriendsList: Starting lookup of %d FriendIds user infos"), FriendIds.Num());
						bIsQueryingUsers = TencentUser->QueryUserInfo(LocalUserNum, FriendIds);
						if (!bIsQueryingUsers)
						{
							Error.bSucceeded = false;
							Error.SetFromErrorCode(TEXT("QueryUserInfo request failed"));
						}
					}
					else
					{
						// Not an error. Just logging for information.
						UE_LOG_ONLINE_FRIEND(VeryVerbose, TEXT("ReadFriendsList: No friends to lookup"));
					}
				}
				else
				{
					Error.SetFromErrorCode(FString::Printf(TEXT("IRailFriends.GetFriendsList failed: %s"), *LexToString(Result)));
				}
			}
			else
			{
				Error.SetFromErrorCode(TEXT("No TencentUser interface"));
			}
		}
		else
		{
			Error.SetFromErrorCode(TEXT("No RailFriends interface"));
		}
	}
	else
	{
		// We don't use ListName, so it will be safe to call all delegates once the request in progress completes.
		// Setting the following will add this delegate to the list to call once the query completes
		Error.bSucceeded = true;
		bIsQueryingUsers = true;
	}

	if (!Error.WasSuccessful())
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("ReadFriendsList request failed. %s"), *Error.ToLogString());
	}

	const bool bSucceeded = Error.WasSuccessful();
	if (!bSucceeded || !bIsQueryingUsers)
	{
		TencentSubsystem->ExecuteNextTick([Delegate, LocalUserNum, ListName, MovedError = MoveTemp(Error)]()
		{
			Delegate.ExecuteIfBound(LocalUserNum, MovedError.WasSuccessful(), ListName, MovedError.GetErrorCode());
		});
	}
	else
	{
		OnReadFriendsListCompleteDelegates.Add(Delegate);
	}

	return bSucceeded;
}

void FOnlineFriendsTencent::OnQueryUsersForFriendsListComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FUniqueNetIdRef>& UserIds, const FString& ErrorStr, FString ListName, TArray<FUniqueNetIdRef> ExpectedFriendIds)
{
	if (UserIds.Num() != ExpectedFriendIds.Num())
	{
		// Make sure we are listening for the query we triggered
		return;
	}

	{
		for (const FUniqueNetIdRef& UserId : UserIds)
		{
			for (int32 FriendIdx = 0; FriendIdx < ExpectedFriendIds.Num(); ++FriendIdx)//FUniqueNetIdRef ExpectedId : ExpectedFriendIds)
			{
				const FUniqueNetIdRef& ExpectedId = ExpectedFriendIds[FriendIdx];
				if (*ExpectedId == *UserId)
				{
					// Not a ref in the iterator because remove doesn't allow the exact memory address
					ExpectedFriendIds.RemoveAtSwap(FriendIdx);
					break;
				}
			}
		}

		if (ExpectedFriendIds.Num() > 0)
		{
			// Make sure we are listening for the query we triggered
			return;
		}
	}

	uint32 Status = ONLINE_FAIL;
	IOnlineUserPtr TencentUser = TencentSubsystem->GetUserInterface();
	if (TencentUser.IsValid())
	{
		TencentUser->ClearOnQueryUserInfoCompleteDelegate_Handle(LocalUserNum, OnQueryUsersForFriendsListCompleteDelegate);
		if (bWasSuccessful)
		{
			IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
			if (IdentityInt.IsValid())
			{
				FUniqueNetIdRailPtr LocalUserId = StaticCastSharedPtr<const FUniqueNetIdRail>(IdentityInt->GetUniquePlayerId(LocalUserNum));
				if (LocalUserId.IsValid())
				{
					// Get/add friends lists for user
					FOnlineFriendsListTencent& FriendList = FriendsLists.FindOrAdd(static_cast<rail::RailID>(*LocalUserId));

					// refresh friends list
					TArray<FUniqueNetIdRef> PresenceUserIds;
					PresenceUserIds.Empty(UserIds.Num());
					FriendList.Friends.Empty(UserIds.Num());
					for (const FUniqueNetIdRef& FriendUserId : UserIds)
					{
						if (*FriendUserId != *LocalUserId)
						{
							TSharedPtr<FOnlineUser> FriendUser = TencentUser->GetUserInfo(LocalUserNum, *FriendUserId);
							if (FriendUser.IsValid())
							{
								FUniqueNetIdRailRef FriendRailUserId = StaticCastSharedRef<const FUniqueNetIdRail>(FriendUserId);

								UE_LOG_ONLINE_FRIEND(VeryVerbose, TEXT("  Id: %s"), *FriendRailUserId->ToDebugString());
								UE_LOG_ONLINE_FRIEND(VeryVerbose, TEXT("  Display name: %s"), *FriendUser->GetDisplayName());

								TSharedRef<FOnlineFriendTencent> FriendEntry = MakeShared<FOnlineFriendTencent>(TencentSubsystem, FriendRailUserId);
								FriendEntry->AccountData.Add(USER_ATTR_DISPLAYNAME, FriendUser->GetDisplayName());

								// Add new friend entry to list
								FriendList.Friends.Add(FriendEntry);
								PresenceUserIds.Add(FriendUserId);
							}
						}
					}

					UE_LOG_ONLINE_FRIEND(Verbose, TEXT("You have %d friends."), FriendList.Friends.Num());

					if (PresenceUserIds.Num() > 0)
					{
						// query presence
						FOnlineAsyncTaskRailQueryFriendsPresence::FCompletionDelegate CompletionDelegate;
						CompletionDelegate.BindThreadSafeSP(this, &FOnlineFriendsTencent::OnQueryFriendsPresenceComplete, LocalUserNum, ListName);
						FOnlineAsyncTaskRailQueryFriendsPresence* AsyncTask = new FOnlineAsyncTaskRailQueryFriendsPresence(TencentSubsystem, PresenceUserIds, CompletionDelegate);
						TencentSubsystem->QueueAsyncParallelTask(AsyncTask);
					}

					Status = PresenceUserIds.Num() > 0 ? ONLINE_IO_PENDING : ONLINE_SUCCESS;
				}
			}
		}
		else
		{
			UE_LOG_ONLINE_FRIEND(Display, TEXT("Query failed with error %s"), *ErrorStr);
		}
	}

	if (Status != ONLINE_IO_PENDING)
	{
		if (Status == ONLINE_SUCCESS)
		{
			// GetPresence calls for the users in this query are now valid
			TriggerOnFriendsChangeDelegates(LocalUserNum);
		}

		TArray<FOnReadFriendsListComplete> Delegates = MoveTemp(OnReadFriendsListCompleteDelegates);
		OnReadFriendsListCompleteDelegates.Empty();
		for (FOnReadFriendsListComplete& Delegate : Delegates)
		{
			Delegate.ExecuteIfBound(LocalUserNum, (Status == ONLINE_SUCCESS), ListName, ErrorStr);
		}
	}
}

void FOnlineFriendsTencent::OnQueryFriendsPresenceComplete(const FQueryUserPresenceTaskResult& TaskResult, int32 LocalUserNum, FString ListName)
{
	if (TaskResult.Error.WasSuccessful())
	{
		// GetPresence calls for the users in this query are now valid
		TriggerOnFriendsChangeDelegates(LocalUserNum);
	}

	TArray<FOnReadFriendsListComplete> Delegates = MoveTemp(OnReadFriendsListCompleteDelegates);
	OnReadFriendsListCompleteDelegates.Empty();
	for (FOnReadFriendsListComplete& Delegate : Delegates)
	{
		Delegate.ExecuteIfBound(LocalUserNum, TaskResult.Error.WasSuccessful(), ListName, TaskResult.Error.GetErrorMessage().ToString());
	}
}

bool FOnlineFriendsTencent::DeleteFriendsList(int32 LocalUserNum, const FString& ListName, const FOnDeleteFriendsListComplete& Delegate /*= FOnDeleteFriendsListComplete()*/)
{
	/** NYI */
	FString ErrorStr;
	bool bStarted = false;
	if (!bStarted)
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("DeleteFriendsList request failed. %s"), *ErrorStr);
		Delegate.ExecuteIfBound(LocalUserNum, false, ListName, ErrorStr);
	}
	return bStarted;
}

bool FOnlineFriendsTencent::SendInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnSendInviteComplete& Delegate)
{
	FString ErrorStr;
	bool bStarted = false;

	const FUniqueNetIdRail& RailFriendId = *static_cast<const FUniqueNetIdRail*>(&FriendId);
	if (RailFriendId.IsValid())
	{
		IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
		if (IdentityInt.IsValid())
		{
			FUniqueNetIdPtr LocalUserId = TencentSubsystem->GetIdentityInterface()->GetUniquePlayerId(LocalUserNum);
			if (LocalUserId.IsValid())
			{
				FOnOnlineAsyncTaskRailAddFriendComplete CompletionDelegate;
				CompletionDelegate.BindThreadSafeSP(this, &FOnlineFriendsTencent::SendInvite_Complete, LocalUserNum, FriendId.AsShared(), ListName, Delegate);
				FOnlineAsyncTaskRailAddFriend* AsyncTask = new FOnlineAsyncTaskRailAddFriend(TencentSubsystem, RailFriendId, CompletionDelegate);
				TencentSubsystem->QueueAsyncTask(AsyncTask);
				bStarted = true;
			}
			else
			{
				ErrorStr = FString::Printf(TEXT("Invalid user %d"), LocalUserNum);
			}
		}
		else
		{
			ErrorStr = TEXT("Identity interface missing");
		}
	}
	else
	{
		ErrorStr = TEXT("Invalid friend id");
	}

	if (!bStarted)
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("SendInvite request failed. %s"), *ErrorStr);

		FUniqueNetIdRef FriendIdRef(FriendId.AsShared());
		TencentSubsystem->ExecuteNextTick([Delegate, LocalUserNum, FriendIdRef, ListName, ErrorStr]()
		{
			Delegate.ExecuteIfBound(LocalUserNum, false, *FriendIdRef, ListName, ErrorStr);
		});
	}
	return bStarted;
}

void FOnlineFriendsTencent::SendInvite_Complete(const FOnlineError& Result, const int32 LocalUserNum, const FUniqueNetIdRef FriendId, const FString ListName, const FOnSendInviteComplete CompletionDelegate)
{
	if (Result.WasSuccessful())
	{
		UE_LOG_ONLINE_FRIEND(Verbose, TEXT("Friend invite successfully sent to %s"), *FriendId->ToDebugString());
	}
	else
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Failed to send friend invitation to %s: %s"), *FriendId->ToString(), *Result.ToLogString());
	}

	TencentSubsystem->ExecuteNextTick([CompletionDelegate, LocalUserNum, FriendId, ListName, Result]()
	{
		CompletionDelegate.ExecuteIfBound(LocalUserNum, Result.WasSuccessful(), *FriendId, ListName, Result.GetErrorCode());
	});
}

bool FOnlineFriendsTencent::AcceptInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnAcceptInviteComplete& Delegate /*= FOnAcceptInviteComplete()*/)
{
	/** NYI */
	FString ErrorStr;
	bool bStarted = false;

	if (!bStarted)
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("AcceptInvite request failed. %s"), *ErrorStr);
		Delegate.ExecuteIfBound(LocalUserNum, false, FriendId, ListName, ErrorStr);
	}
	return bStarted;
}

bool FOnlineFriendsTencent::RejectInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	/** NYI */
	FString ErrorStr;
	bool bStarted = false;

	if (!bStarted)
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("RejectInvite request failed. %s"), *ErrorStr);
		TriggerOnRejectInviteCompleteDelegates(LocalUserNum, false, FriendId, ListName, ErrorStr);
	}
	return bStarted;
}

void FOnlineFriendsTencent::SetFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FString& Alias, const FOnSetFriendAliasComplete& Delegate /*= FOnSetFriendAliasComplete()*/)
{
	FUniqueNetIdRef FriendIdRef = FriendId.AsShared();
	TencentSubsystem->ExecuteNextTick([LocalUserNum, FriendIdRef, ListName, Delegate]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsTencent::SetFriendAlias is not implemented"));
		Delegate.ExecuteIfBound(LocalUserNum, *FriendIdRef, ListName, FOnlineError(EOnlineErrorResult::NotImplemented));
	});
}

void FOnlineFriendsTencent::DeleteFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnDeleteFriendAliasComplete& Delegate)
{
	FUniqueNetIdRef FriendIdRef = FriendId.AsShared();
	TencentSubsystem->ExecuteNextTick([LocalUserNum, FriendIdRef, ListName, Delegate]()
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsTencent::DeleteFriendAlias is not implemented"));
		Delegate.ExecuteIfBound(LocalUserNum, *FriendIdRef, ListName, FOnlineError(EOnlineErrorResult::NotImplemented));
	});
}

bool FOnlineFriendsTencent::DeleteFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	/** NYI */
	FString ErrorStr;
	bool bStarted = false;

	if (!bStarted)
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("DeleteFriend request failed. %s"), *ErrorStr);
		TriggerOnDeleteFriendCompleteDelegates(LocalUserNum, false, FriendId, ListName, ErrorStr);
	}
	return bStarted;
}

bool FOnlineFriendsTencent::GetFriendsList(int32 LocalUserNum, const FString& ListName, TArray< TSharedRef<FOnlineFriend> >& OutFriends)
{
	IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		FUniqueNetIdRailPtr UserId = StaticCastSharedPtr<const FUniqueNetIdRail>(IdentityInt->GetUniquePlayerId(LocalUserNum));
		if (UserId.IsValid())
		{
			FOnlineFriendsListTencent* FriendList = FriendsLists.Find(static_cast<rail::RailID>(*UserId));
			if (FriendList != nullptr)
			{
				OutFriends.Empty(FriendList->Friends.Num());
				for (const TSharedRef<FOnlineFriendTencent>& Friend : FriendList->Friends)
				{
					OutFriends.Add(Friend);
				}
				return true;
			}
			else
			{
				UE_LOG_ONLINE_FRIEND(Verbose, TEXT("FOnlineFriendsTencent::GetFriendsList - Unable to find friends lists for %s"), *UserId->ToDebugString());
			}
		}
		else
		{
			UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsTencent::GetFriendsList - UserId invalid"));
		}
	}
	else
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsTencent::GetFriendsList - OssIdentityTencent invalid"));
	}

	OutFriends.Empty();
	return false;
}

TSharedPtr<FOnlineFriend> FOnlineFriendsTencent::GetFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	return FindFriendEntry(LocalUserNum, (const FUniqueNetIdRail&)FriendId, false);
}

bool FOnlineFriendsTencent::IsFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName)
{
	TSharedPtr<const FOnlineFriend> TencentFriend = GetFriend(LocalUserNum, FriendId, ListName);
	const bool bIsFriend = TencentFriend.IsValid() && TencentFriend->GetInviteStatus() == EInviteStatus::Accepted;
	return bIsFriend;
}

void FOnlineFriendsTencent::AddRecentPlayers(const FUniqueNetId& UserId, const TArray<FReportPlayedWithUser>& InRecentPlayers, const FString& ListName, const FOnAddRecentPlayersComplete& InCompletionDelegate)
{
	bool bTriggerDelegate = true;
	FOnlineError Error(EOnlineErrorResult::Unknown);

	FUniqueNetIdRef UserIdRef = UserId.AsShared();
	if (InRecentPlayers.Num() > 0)
	{
		FOnlineIdentityTencentPtr IdentityInt = StaticCastSharedPtr<FOnlineIdentityTencent>(TencentSubsystem->GetIdentityInterface());

		TArray<FReportPlayedWithUser> RecentPlayersCopy = InRecentPlayers;
		RecentPlayersCopy.RemoveAll([IdentityInt](const FReportPlayedWithUser& Candidate)
		{
			// Remove local players from report
			int32 LocalUserNum = INDEX_NONE;
			return IdentityInt->GetLocalUserIdx(*Candidate.UserId, LocalUserNum);
		});

		if (RecentPlayersCopy.Num() > 0)
		{
			Error = FOnlineError::Success();

			static FTimespan TenMinutes(10 * ETimespan::TicksPerMinute);
			const FDateTime TenMinutesAgo = FDateTime::UtcNow() - TenMinutes;

			// Remove recent players that have already been reported
			TArray<TSharedRef<FOnlineRecentPlayerTencent>>& RecentPlayersMap = RecentPlayersLists.FindOrAdd(UserIdRef);
			for (const TSharedRef<FOnlineRecentPlayerTencent>& CachedPlayer : RecentPlayersMap)
			{
				for (int32 PlayerIdx = 0; PlayerIdx < RecentPlayersCopy.Num(); ++PlayerIdx)
				{
					if ((*CachedPlayer->GetUserId() == *RecentPlayersCopy[PlayerIdx].UserId) &&
						(CachedPlayer->LastSeen > TenMinutesAgo))
					{
						RecentPlayersCopy.RemoveAtSwap(PlayerIdx);
						break;
					}
				}
			}

			if (RecentPlayersCopy.Num() > 0)
			{
				TWeakPtr<FOnlineFriendsTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
				FOnlineAsyncTaskRailReportPlayedWithUsers* NewTask = new FOnlineAsyncTaskRailReportPlayedWithUsers(TencentSubsystem, RecentPlayersCopy, FOnOnlineAsyncTaskRailReportPlayedWithUsersComplete::CreateLambda([LocalWeakThis, UserIdRef, InCompletionDelegate](const FReportPlayedWithUsersTaskResult& Result)
				{
					if (Result.Error.WasSuccessful())
					{
						FOnlineFriendsTencentPtr StrongThis = StaticCastSharedPtr<FOnlineFriendsTencent>(LocalWeakThis.Pin());
						if (StrongThis.IsValid())
						{
							TArray<TSharedRef<FOnlineRecentPlayerTencent>>& RecentPlayersMap = StrongThis->RecentPlayersLists.FindOrAdd(UserIdRef);
							for (const FReportPlayedWithUser& UserReport : Result.UsersReported)
							{
								TSharedRef<FOnlineRecentPlayerTencent>* FoundPlayer = RecentPlayersMap.FindByPredicate([UserReport](const TSharedRef<FOnlineRecentPlayerTencent>& Candidate)
								{ 
									return *Candidate->GetUserId() == *UserReport.UserId;
								});

								if (FoundPlayer)
								{
									(*FoundPlayer)->LastSeen = FDateTime::UtcNow();
								}
								else
								{
									TSharedRef<FOnlineRecentPlayerTencent> RecentPlayer = MakeShared<FOnlineRecentPlayerTencent>(UserReport.UserId);
									RecentPlayer->LastSeen = FDateTime::UtcNow();
									RecentPlayersMap.Emplace(RecentPlayer);
								}
							}
						}
					}
					InCompletionDelegate.ExecuteIfBound(*UserIdRef, Result.Error);
				}));

				TencentSubsystem->QueueAsyncTask(NewTask);
				bTriggerDelegate = false;
			}
			else
			{
				Error.SetFromErrorCode(TEXT("only_cached_players"));
			}
		}
		else
		{
			Error.SetFromErrorCode(TEXT("only_local_players"));
			Error.bSucceeded = true;
		}
	}
	else
	{
		Error.SetFromErrorCode(TEXT("no_recent_players"));
	}
	
	if (bTriggerDelegate)
	{
		TencentSubsystem->ExecuteNextTick([UserIdRef, MoveError = MoveTemp(Error), InCompletionDelegate]()
		{
			InCompletionDelegate.ExecuteIfBound(*UserIdRef, MoveError);
		});
	}
}

bool FOnlineFriendsTencent::QueryRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace)
{
	FOnlineError Error(EOnlineErrorResult::Unknown);
	bool bStarted = false;
	if (UserId.IsValid())
	{
		FUniqueNetIdRef UserIdRef = UserId.AsShared();
		TWeakPtr<FOnlineFriendsTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
		FOnlineAsyncTaskRailQueryPlayedWithFriendsList* NewTask = new FOnlineAsyncTaskRailQueryPlayedWithFriendsList(TencentSubsystem, FOnOnlineAsyncTaskRailQueryPlayedWithFriendsListComplete::CreateLambda([LocalWeakThis, UserIdRef, Namespace](const FQueryPlayedWithFriendsListTaskResult& Result)
		{
			FOnlineFriendsTencentPtr StrongThis = StaticCastSharedPtr<FOnlineFriendsTencent>(LocalWeakThis.Pin());
			if (StrongThis.IsValid())
			{
				if (Result.Error.WasSuccessful())
				{
					if (Result.UsersPlayedWith.Num() > 0)
					{
						FOnlineAsyncTaskRailQueryPlayedWithFriendsTime* InnerTask = new FOnlineAsyncTaskRailQueryPlayedWithFriendsTime(StrongThis->TencentSubsystem, Result.UsersPlayedWith, FOnOnlineAsyncTaskRailQueryPlayedWithFriendsTimeComplete::CreateLambda([LocalWeakThis, UserIdRef, Namespace](const FQueryPlayedWithFriendsTimeTaskResult& Result)
						{
							FOnlineFriendsTencentPtr StrongThis = StaticCastSharedPtr<FOnlineFriendsTencent>(LocalWeakThis.Pin());
							if (StrongThis.IsValid())
							{
								if (Result.Error.WasSuccessful())
								{
									TArray<TSharedRef<FOnlineRecentPlayerTencent>>& RecentPlayerList = StrongThis->RecentPlayersLists.FindOrAdd(UserIdRef);
									for (const FQueryPlayedWithFriendsTimeTaskResult::FRecentPlayers& LastPlayedWith : Result.LastPlayedWithUsers)
									{
										TSharedPtr<FOnlineRecentPlayerTencent> ExistingPlayer = nullptr;
										for (TSharedRef<FOnlineRecentPlayerTencent> RecentPlayer : RecentPlayerList)
										{
											if (*RecentPlayer->GetUserId() == *LastPlayedWith.UserId)
											{
												ExistingPlayer = RecentPlayer;
												break;
											}
										}

										if (!ExistingPlayer.IsValid())
										{
											ExistingPlayer = MakeShared<FOnlineRecentPlayerTencent>(LastPlayedWith.UserId);
											RecentPlayerList.Add(ExistingPlayer.ToSharedRef());
										}

										ExistingPlayer->LastSeen = LastPlayedWith.LastPlayed;
									}
								}

								StrongThis->TriggerOnQueryRecentPlayersCompleteDelegates(*UserIdRef, Namespace, Result.Error.WasSuccessful(), Result.Error.GetErrorMessage().ToString());
							}
						}));
						StrongThis->TencentSubsystem->QueueAsyncTask(InnerTask);
					}
					else
					{
						StrongThis->TriggerOnQueryRecentPlayersCompleteDelegates(*UserIdRef, Namespace, true, FString());
					}
				}
				else
				{
					StrongThis->TriggerOnQueryRecentPlayersCompleteDelegates(*UserIdRef, Namespace, false, Result.Error.GetErrorMessage().ToString());
				}
			}
		}));

		TencentSubsystem->QueueAsyncTask(NewTask);
		Error = FOnlineError::Success();
	}
	else
	{
		Error.SetFromErrorCode(TEXT("invalid_user_id"));
	}
	
	if (!Error.WasSuccessful())
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("QueryRecentPlayers request failed. %s"), *Error.ToLogString());
		TriggerOnQueryRecentPlayersCompleteDelegates(UserId, Namespace, false, Error.GetErrorMessage().ToString());
	}

	return Error.WasSuccessful();
}

bool FOnlineFriendsTencent::GetRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace, TArray< TSharedRef<FOnlineRecentPlayer> >& OutRecentPlayers)
{
	/** NYI */
	OutRecentPlayers.Empty();

	if (UserId.IsValid())
	{
	}
	else
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsTencent::GetRecentPlayers - UserId invalid"));
	}
	return false;
}

void FOnlineFriendsTencent::DumpRecentPlayers() const
{
	UE_LOG_ONLINE_FRIEND(Display, TEXT("Recent Players"));
	for (const TPair<FUniqueNetIdRef, TArray<TSharedRef<FOnlineRecentPlayerTencent>>>& SingleUserList : RecentPlayersLists)
	{
		const FUniqueNetIdRef& UserId = SingleUserList.Key;
		UE_LOG_ONLINE_FRIEND(Display, TEXT("LocalUser: %s"), *UserId->ToDebugString());
		for (const TSharedRef<FOnlineRecentPlayerTencent>& RecentPlayer : SingleUserList.Value)
		{
			UE_LOG_ONLINE_FRIEND(Display, TEXT("    User: %s"), *RecentPlayer->GetUserId()->ToDebugString());
			UE_LOG_ONLINE_FRIEND(Display, TEXT("    LastSeen: %s"), *RecentPlayer->GetLastSeen().ToString());
		}
	}
}

bool FOnlineFriendsTencent::BlockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId)
{
	/** NYI */
	FString ErrorStr;
	bool bStarted = false;

	if (!bStarted)
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Block Player request failed. %s"), *ErrorStr);
	}
	return bStarted;
}

bool FOnlineFriendsTencent::UnblockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId)
{
	/** NYI */
	FString ErrorStr;
	bool bStarted = false;

	if (!bStarted)
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("Unblock request failed. %s"), *ErrorStr);
	}
	return bStarted;
}

bool FOnlineFriendsTencent::QueryBlockedPlayers(const FUniqueNetId& UserId)
{
	/** NYI */
	FString ErrorStr;
	bool bStarted = false;

	if (!bStarted)
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("QueryBlockedPlayers request failed. %s"), *ErrorStr);
		TriggerOnQueryBlockedPlayersCompleteDelegates(UserId, false, ErrorStr);
	}
	return bStarted;
}

bool FOnlineFriendsTencent::GetBlockedPlayers(const FUniqueNetId& UserId, TArray< TSharedRef<FOnlineBlockedPlayer> >& OutBlockedPlayers)
{
	/** NYI */
	OutBlockedPlayers.Empty();

	if (UserId.IsValid())
	{
	}
	else
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsTencent::GetBlockedPlayers - UserId invalid"));
	}
	return false;
}

bool FOnlineFriendsTencent::IsPlayerBlocked(const FUniqueNetId& LocalUserId, const FUniqueNetId& PeerUserId) const
{
	/** NYI */
	if (LocalUserId.IsValid() && PeerUserId.IsValid())
	{
	}
	else
	{
		UE_LOG_ONLINE_FRIEND(Warning, TEXT("FOnlineFriendsTencent::IsPlayerBlocked - LocalUserId %s, PeerUserId %s"),
			LocalUserId.IsValid() ? TEXT("valid") : TEXT("invalid"),
			PeerUserId.IsValid() ? TEXT("valid") : TEXT("invalid"));
	}
	return false;
}

void FOnlineFriendsTencent::OnRailFriendsListChanged(const FUniqueNetIdRail& UserId)
{
	// Re-read the friends list
	IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		const FPlatformUserId PlatformUserId = IdentityInt->GetPlatformUserIdFromUniqueNetId(UserId);
		static const FString FriendsListName(TEXT("")); // Use an empty list name
		static const FOnReadFriendsListComplete CompletionDelegate; // Use an empty completion delegate
		const bool bReadFriendsListResult = ReadFriendsList(PlatformUserId, FriendsListName, CompletionDelegate);
		if (!bReadFriendsListResult)
		{
			UE_LOG_ONLINE_FRIEND(Verbose, TEXT("FOnlineFriendsTencent::OnRailFriendsListChanged: ReadFriendsList failed"));
		}
	}
	else
	{
		UE_LOG_ONLINE_FRIEND(Verbose, TEXT("FOnlineFriendsTencent::OnRailFriendsListChanged: No identity interface"));
	}
}

TSharedPtr<FOnlineFriendTencent> FOnlineFriendsTencent::FindFriendEntry(int32 LocalUserNum, const FUniqueNetIdRail& FriendId, bool bAddIfMissing)
{
	TSharedPtr<FOnlineFriendTencent> Result;
	IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
	FUniqueNetIdRailPtr UserId = StaticCastSharedPtr<const FUniqueNetIdRail>(IdentityInt->GetUniquePlayerId(LocalUserNum));
	if (UserId.IsValid())
	{
		// Find the friends lists available for the local user
		FOnlineFriendsListTencent* FriendList = FriendsLists.Find((rail::RailID)(*UserId));
		// Create the friends lists for the user if missing
		if (bAddIfMissing && FriendList == nullptr)
		{
			FriendList = &FriendsLists.Add((rail::RailID)(*UserId), FOnlineFriendsListTencent());
		}
		if (FriendList != nullptr)
		{
			// Find the friend entry in the friends list
			for (auto Friend : FriendList->Friends)
			{
				if (*Friend->GetUserId() == FriendId)
				{
					Result = Friend;
					break;
				}
			}
			// Create friend entry if missing
			if (bAddIfMissing && !Result.IsValid())
			{
				FUniqueNetIdRailRef FriendUserId = StaticCastSharedRef<const FUniqueNetIdRail>(FriendId.AsShared());
				TSharedRef<FOnlineFriendTencent> FriendEntry = MakeShared<FOnlineFriendTencent>(TencentSubsystem, FriendUserId);
				FriendList->Friends.Add(FriendEntry);
				Result = FriendEntry;
			}
		}
	}
	return Result;
}

void FOnlineFriendsTencent::DumpBlockedPlayers() const
{
	UE_LOG_ONLINE_FRIEND(Display, TEXT("Blocked Players NYI"));
}

#endif // WITH_TENCENT_RAIL_SDK
#endif // WITH_TENCENTSDK
