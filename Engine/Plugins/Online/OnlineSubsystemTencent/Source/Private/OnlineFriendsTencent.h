// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TENCENTSDK
#if WITH_TENCENT_RAIL_SDK

#include "RailSDK.h"
#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystemTencentTypes.h"
#include "OnlineAsyncTasksTencent.h"
#include "OnlineSubsystemTencentPackage.h"

class FOnlineSubsystemTencent;
class FOnlineUserPresence;

struct FOnlineError;

/**
 * Info associated with an online friend on the Tencent / Rail service
 */
class FOnlineFriendTencent : public FOnlineFriend
{
public:

	// FOnlineUser

	virtual FUniqueNetIdRef GetUserId() const override;
	virtual FString GetRealName() const override;
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;

	// FOnlineFriend

	virtual EInviteStatus::Type GetInviteStatus() const override;
	virtual const FOnlineUserPresence& GetPresence() const override;

	/**
	 * Init constructor
	 */
	FOnlineFriendTencent(FOnlineSubsystemTencent* InTencentSubsystem, const FUniqueNetIdRailRef InUserId);

	/**
	 * Destructor
	 */
	virtual ~FOnlineFriendTencent()
	{
	}

	/**
	 * Get account data attribute
	 *
	 * @param Key account data entry key
	 * @param OutVal [out] value that was found
	 *
	 * @return true if entry was found
	 */
	inline bool GetAccountData(const FString& Key, FString& OutVal) const
	{
		const FString* FoundVal = AccountData.Find(Key);
		if (FoundVal != NULL)
		{
			OutVal = *FoundVal;
			return true;
		}
		return false;
	}

	/** Used to access presence data for the friend entry */
	FOnlineSubsystemTencent* TencentSubsystem;
	/** User Id represented as a FUniqueNetId */
	FUniqueNetIdRailRef UserId;
	/** Any addition account data associated with the friend */
	TMap<FString, FString> AccountData;
};

/**
 * Named list of friends
 */
class FOnlineFriendsListTencent
{
public:

	/**
	* Init/default constructor
	*/
	FOnlineFriendsListTencent(const FString& InListName = TEXT(""))
		: ListName(InListName)
	{
	}

	/** Can have multiple friends lists. Each with its own name */
	FString ListName;
	/** Array of friends populated by ReadFriendsList */
	TArray<TSharedRef<FOnlineFriendTencent> > Friends;
};
/** Mapping from user id to array of friends lists */
typedef TRailIdMap<FOnlineFriendsListTencent> FOnlineFriendsListTencentMap;

/**
 * Info associated with a recent player on the Tencent service
 */
class FOnlineRecentPlayerTencent :
	public FOnlineRecentPlayer
{
public:

	// FOnlineUser

	virtual FUniqueNetIdRef GetUserId() const override;
	virtual FString GetRealName() const override;
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;

	// FOnlineRecentPlayer

	virtual FDateTime GetLastSeen() const override;

	/**
	 * Init/default constructor
	 */
	FOnlineRecentPlayerTencent(const FUniqueNetId& InUserId)
		: UserId(InUserId.AsShared())
	{
	}

	FOnlineRecentPlayerTencent(const FUniqueNetIdRef& InUserId)
		: UserId(InUserId)
	{
	}

	FOnlineRecentPlayerTencent(const FString& InUserId = TEXT(""))
		: UserId(FUniqueNetIdRail::Create(InUserId))
	{
	}

	/**
	 * Destructor
	 */
	virtual ~FOnlineRecentPlayerTencent()
	{
	}

	/**
	 * Get account data attribute
	 *
	 * @param Key account data entry key
	 * @param OutVal [out] value that was found
	 *
	 * @return true if entry was found
	 */
	inline bool GetAccountData(const FString& Key, FString& OutVal) const
	{
		const FString* FoundVal = AccountData.Find(Key);
		if (FoundVal != NULL)
		{
			OutVal = *FoundVal;
			return true;
		}
		return false;
	}

	/** User Id represented as a FUniqueNetId */
	FUniqueNetIdRef UserId;
	/** last seen in online game */
	FDateTime LastSeen;
	/** Any addition account data associated with the recent player */
	TMap<FString, FString> AccountData;
};

/** Mapping from user id to list of recent players */
typedef TUniqueNetIdMap<TArray<TSharedRef<FOnlineRecentPlayerTencent>>> FOnlineRecentPlayersTencentMap;

/**
 * Tencent / Rail implementation of the online friends interface
 */
class FOnlineFriendsTencent :
	public IOnlineFriends,
	public TSharedFromThis<FOnlineFriendsTencent, ESPMode::ThreadSafe>
{
public:

	//~ Begin IOnlineFriends Interface
	virtual bool ReadFriendsList(int32 LocalUserNum, const FString& ListName, const FOnReadFriendsListComplete& Delegate = FOnReadFriendsListComplete()) override;
	virtual bool DeleteFriendsList(int32 LocalUserNum, const FString& ListName, const FOnDeleteFriendsListComplete& Delegate = FOnDeleteFriendsListComplete()) override;
	virtual bool SendInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnSendInviteComplete& Delegate = FOnSendInviteComplete()) override;
	virtual bool AcceptInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnAcceptInviteComplete& Delegate = FOnAcceptInviteComplete()) override;
	virtual bool RejectInvite(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual void SetFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FString& Alias, const FOnSetFriendAliasComplete& Delegate = FOnSetFriendAliasComplete()) override;
	virtual void DeleteFriendAlias(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName, const FOnDeleteFriendAliasComplete& Delegate = FOnDeleteFriendAliasComplete()) override;
	virtual bool DeleteFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool GetFriendsList(int32 LocalUserNum, const FString& ListName, TArray< TSharedRef<FOnlineFriend> >& OutFriends) override;
	virtual TSharedPtr<FOnlineFriend> GetFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual bool IsFriend(int32 LocalUserNum, const FUniqueNetId& FriendId, const FString& ListName) override;
	virtual void AddRecentPlayers(const FUniqueNetId& UserId, const TArray<FReportPlayedWithUser>& InRecentPlayers, const FString& ListName, const FOnAddRecentPlayersComplete& InCompletionDelegate);
	virtual bool QueryRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace) override;
	virtual bool GetRecentPlayers(const FUniqueNetId& UserId, const FString& Namespace, TArray< TSharedRef<FOnlineRecentPlayer> >& OutRecentPlayers) override;
	virtual void DumpRecentPlayers() const override;
	virtual bool BlockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) override;
	virtual bool UnblockPlayer(int32 LocalUserNum, const FUniqueNetId& PlayerId) override;
	virtual bool QueryBlockedPlayers(const FUniqueNetId& UserId) override;
	virtual bool GetBlockedPlayers(const FUniqueNetId& UserId, TArray< TSharedRef<FOnlineBlockedPlayer> >& OutBlockedPlayers) override;
	virtual void DumpBlockedPlayers() const override;
	//~ End IOnlineFriends Interface

	// FOnlineFriendsTencent

	/**
	 * Constructor
	 *
	 * @param InSubsystem subsystem being used
	 */
	FOnlineFriendsTencent(FOnlineSubsystemTencent* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlineFriendsTencent();

	/**
	 * Initialize the interface
	 *
	 * @return true if successful, false otherwise
	 */
	bool Init();

	/**
	 * Check to see if a single player is blocked.  Used as an alternative to GetBlockedPlayers which involves array copying
	 *
	 * @param LocalUserId User whose block list we will be checking against
	 * @param PeerUserId User who we want to check if they are blocked
	 *
	 * @return true if the peer user is blocked, false if not
	 */
	bool IsPlayerBlocked(const FUniqueNetId& LocalUserId, const FUniqueNetId& PeerUserId) const;

	/**
	 * Called when Rail informs us the friends list has changed
	 */
	void OnRailFriendsListChanged(const FUniqueNetIdRail& UserId);

private:

	/**
	 * Delegate fired when query user info is complete for a list of Tencent friends
	 *
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param bWasSuccessful true if the async action completed without error, false if there was an error
	 * @param UserIds list of user ids that were queried
	 * @param ErrorStr string representing the error condition
	 * @param ListName name of the friends list to use
	 */
	void OnQueryUsersForFriendsListComplete(int32 LocalUserNum, bool bWasSuccessful, const TArray<FUniqueNetIdRef>& UserIds, const FString& ErrorStr, FString ListName, TArray<FUniqueNetIdRef> ExpectedFriendIds);

	/**
	 * Delegate fired when query presence is complete for a list of Tencent friends
	 *
	 * @param TaskResult result of the async action
	 * @param LocalUserNum the controller number of the associated user that made the request
	 * @param ListName name of the friends list to use
	 */
	void OnQueryFriendsPresenceComplete(const FQueryUserPresenceTaskResult& TaskResult, int32 LocalUserNum, FString ListName);

	/**
	 * Notification that login status has changed for a user
	 *
	 * @param LocalUserNum id of local user whose login status has changed
	 */
	void OnLoginChanged(int32 LocalUserNum);

	/**
	 * Find an existing friend entry that is cached
	 *
	 * @param LocalUserNum user with friends list to operate on
	 * @param FriendId id of friend to find
	 * @param bAddIfMissing add new entry to cache if true
	 *
	 * @return friend entry pointer if found or NULL if not found
	 */
	TSharedPtr<FOnlineFriendTencent> FindFriendEntry(int32 LocalUserNum, const FUniqueNetIdRail& FriendId, bool bAddIfMissing);

	/** Called when the SendInvite async task has completed */
	void SendInvite_Complete(const FOnlineError& Result, const int32 LocalUserNum, const FUniqueNetIdRef FriendId, const FString ListName, const FOnSendInviteComplete CompletionDelegate);
	
	/**
	 * Should use the initialization constructor instead
	 */
	FOnlineFriendsTencent() = delete;

	FOnlineSubsystemTencent* TencentSubsystem;

	/** Cache of all friends lists for all local users */
	FOnlineFriendsListTencentMap FriendsLists;

	/** Handle to login change events for clearing friends lists */
	FDelegateHandle OnLoginChangedHandle;
	/** Handle when a query user info call for reading friends list is complete */
	FDelegateHandle OnQueryUsersForFriendsListCompleteDelegate;

	/** Cache of all recent players lists for all local users */
	FOnlineRecentPlayersTencentMap RecentPlayersLists;

	/** Array of delegates for ReadFriendList requests. All will be completed at the same time if there were calls to ReadFriendsList while it was in progress */
	TArray<FOnReadFriendsListComplete> OnReadFriendsListCompleteDelegates;
};

typedef TSharedPtr<FOnlineFriendsTencent, ESPMode::ThreadSafe> FOnlineFriendsTencentPtr;


#endif // WITH_TENCENT_RAIL_SDK
#endif // WITH_TENCENTSDK
