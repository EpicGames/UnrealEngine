// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineIdentityTencent.h"

#if WITH_TENCENT_RAIL_SDK
bool FOnlineIdentityTencent::GetLocalUserIdx(const FUniqueNetId& UserId, int32& OutLocalIdx) const
{
	for (TMap<int32, FUniqueNetIdPtr >::TConstIterator It(UserIds); It; ++It)
	{
		if (*It->Value == UserId)
		{
			OutLocalIdx = It->Key;
			return true;
		}
	}
	return false;
}
#endif


#if !WITH_TENCENT_RAIL_SDK
FOnlineIdentityTencent::FOnlineIdentityTencent(FOnlineSubsystemTencent* InSubsystem) {}
FOnlineIdentityTencent::~FOnlineIdentityTencent() {}
TSharedPtr<FUserOnlineAccountTencent> FOnlineIdentityTencent::GetUserAccountTencent(const FUniqueNetId& UserId) const { return nullptr; }
bool FOnlineIdentityTencent::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials){ return false; }
bool FOnlineIdentityTencent::Logout(int32 LocalUserNum){ return false; }
bool FOnlineIdentityTencent::AutoLogin(int32 LocalUserNum){ return false; }
TSharedPtr<FUserOnlineAccount> FOnlineIdentityTencent::GetUserAccount(const FUniqueNetId& UserId) const{ return nullptr; }
TArray<TSharedPtr<FUserOnlineAccount> > FOnlineIdentityTencent::GetAllUserAccounts() const{ return TArray<TSharedPtr<FUserOnlineAccount> >(); }
FUniqueNetIdPtr FOnlineIdentityTencent::GetUniquePlayerId(int32 LocalUserNum) const { return nullptr; }
FUniqueNetIdPtr FOnlineIdentityTencent::CreateUniquePlayerId(uint8* Bytes, int32 Size){ return nullptr; }
FUniqueNetIdPtr FOnlineIdentityTencent::CreateUniquePlayerId(const FString& Str){ return nullptr; }
ELoginStatus::Type FOnlineIdentityTencent::GetLoginStatus(int32 LocalUserNum) const{ return ELoginStatus::NotLoggedIn; }
ELoginStatus::Type FOnlineIdentityTencent::GetLoginStatus(const FUniqueNetId& UserId) const{ return ELoginStatus::NotLoggedIn; }
FString FOnlineIdentityTencent::GetPlayerNickname(int32 LocalUserNum) const{ return FString(); }
FString FOnlineIdentityTencent::GetPlayerNickname(const FUniqueNetId& UserId) const{ return FString(); }
FString FOnlineIdentityTencent::GetAuthToken(int32 LocalUserNum) const{ return FString(); }
void FOnlineIdentityTencent::GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI){}
FPlatformUserId FOnlineIdentityTencent::GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const { return FPlatformUserId();  }
FString FOnlineIdentityTencent::GetAuthType() const{ return FString(); }
void FOnlineIdentityTencent::RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate){ }
bool FOnlineIdentityTencent::GetLocalUserIdx(const FUniqueNetId& UserId, int32& OutLocalIdx) const { return false; }

#endif // !WITH_TENCENT_RAIL_SDK
