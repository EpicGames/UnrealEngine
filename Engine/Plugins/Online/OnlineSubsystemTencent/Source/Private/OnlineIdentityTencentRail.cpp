// Copyright Epic Games, Inc. All Rights Reserved.


#if WITH_TENCENTSDK
#if WITH_TENCENT_RAIL_SDK

#include "RailSdkWrapper.h"
#include "OnlineAsyncTasksTencent.h"
#include "OnlineIdentityTencent.h"
#include "OnlineSubsystemTencent.h"
#include "OnlineSubsystemTencentPrivate.h"

// FUserOnlineAccountTencent

// Per request from Tencent - when WeGame does not provide us with a display name, use 'Player One'
static const FString InvalidWeGameDisplayName(TEXT("Player One"));

FUniqueNetIdRef FUserOnlineAccountTencent::GetUserId() const
{
	return UserId;
}

FString FUserOnlineAccountTencent::GetRealName() const
{
	FString Result;
	GetAccountData(TEXT("name"), Result);
	return Result;
}

FString FUserOnlineAccountTencent::GetDisplayName(const FString& Platform) const
{
	FString Result;
	GetAccountData(USER_ATTR_DISPLAYNAME, Result);
	return Result;
}

bool FUserOnlineAccountTencent::GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	if (AttrName.Equals(USER_ATTR_ID, ESearchCase::IgnoreCase))
	{
		OutAttrValue = UserId->ToString();
		return true;
	}

	return GetAccountData(AttrName, OutAttrValue);
}

bool FUserOnlineAccountTencent::SetUserAttribute(const FString& AttrName, const FString& AttrValue)
{
	return SetAccountData(AttrName, AttrValue);
}

FString FUserOnlineAccountTencent::GetAccessToken() const
{
	return AuthToken;
}

bool FUserOnlineAccountTencent::GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const
{
	return GetUserAttribute(AttrName, OutAttrValue);
}

TSharedPtr<FUserOnlineAccountTencent> FOnlineIdentityTencent::GetUserAccountTencent(const FUniqueNetId& UserId) const
{
	TSharedPtr<FUserOnlineAccountTencent> Result;

	if (UserAccount.IsValid() &&
		*UserAccount->GetUserId() == UserId)
	{
		Result = UserAccount;
	}

	return Result;
}

TSharedPtr<FUserOnlineAccount> FOnlineIdentityTencent::GetUserAccount(const FUniqueNetId& UserId) const
{
	return GetUserAccountTencent(UserId);
}

TArray<TSharedPtr<FUserOnlineAccount> > FOnlineIdentityTencent::GetAllUserAccounts() const
{
	TArray<TSharedPtr<FUserOnlineAccount> > Result;

	if (UserAccount.IsValid())
	{
		Result.Add(UserAccount);
	}

	return Result;
}

FUniqueNetIdPtr FOnlineIdentityTencent::GetUniquePlayerId(int32 LocalUserNum) const
{
	if (LocalUserNum >= 0 && LocalUserNum < MAX_LOCAL_PLAYERS)
	{
		const FUniqueNetIdPtr* FoundId = UserIds.Find(LocalUserNum);
		if (FoundId != nullptr)
		{
			return *FoundId;
		}
	}
	return nullptr;
}

FUniqueNetIdPtr FOnlineIdentityTencent::CreateUniquePlayerId(uint8* Bytes, int32 Size)
{
	if (Bytes != nullptr && Size == sizeof(rail::RailID))
	{
		return FUniqueNetIdRail::Create(*reinterpret_cast<const rail::RailID*>(Bytes));
	}
	return nullptr;
}

FUniqueNetIdPtr FOnlineIdentityTencent::CreateUniquePlayerId(const FString& Str)
{
	return FUniqueNetIdRail::Create(Str);
}

bool FOnlineIdentityTencent::Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials)
{
	FString ErrorStr;

	UE_LOG_ONLINE_IDENTITY(Log, TEXT("Login starting for user %d"), LocalUserNum);

	// valid local player index
	if (LocalUserNum < 0 || LocalUserNum >= MAX_LOCAL_PLAYERS)
	{
		ErrorStr = FString::Printf(TEXT("Invalid LocalUserNum=%d"), LocalUserNum);
	}

	// Check RailSdk
	rail::IRailPlayer* RailPlayer = nullptr;
	if (ErrorStr.IsEmpty())
	{
		RailSdkWrapper& RailSDK = RailSdkWrapper::Get();
		if (RailSDK.IsInitialized())
		{
			rail::IRailFactory* const RailFactory = RailSDK.RailFactory();
			if (RailFactory)
			{
				RailPlayer = RailFactory->RailPlayer();
				if (!RailPlayer)
				{
					ErrorStr = TEXT("Failed to get RailPlayer");
				}
			}
			else
			{
				ErrorStr = TEXT("Failed to get RailFactory");
			}
		}
		else
		{
			ErrorStr = TEXT("Failed to initialize Rail");
		}
	}

	if (ErrorStr.IsEmpty())
	{
		const bool bIsNewAccount = !UserAccount.IsValid();
		const rail::RailID RailId = RailPlayer->GetRailID();
		if (bIsNewAccount)
		{
			FUniqueNetIdRef UserId = FUniqueNetIdRail::Create(RailId);
			UserAccount = MakeShared<FUserOnlineAccountTencent>(UserId);
			// Try to get the display name
			rail::RailString RailPlayerName;
			rail::RailResult RailPlayerNameResult = RailPlayer->GetPlayerName(&RailPlayerName);
			if (RailPlayerNameResult == rail::RailResult::kSuccess)
			{
				FString PlayerName = LexToString(RailPlayerName);
				UserAccount->AccountData.Emplace(USER_ATTR_DISPLAYNAME, MoveTemp(PlayerName));
			}
			else
			{
				UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Failed to get RailPlayer PlayerName with result %s"), *LexToString(RailPlayerNameResult));
			}

			// keep track of user ids for local users
			UserIds.Emplace(LocalUserNum, UserAccount->GetUserId());
		}

		FOnlineAsyncTaskRailAcquireSessionTicket::FCompletionDelegate CompletionDelegate;
		CompletionDelegate.BindThreadSafeSP(this, &FOnlineIdentityTencent::OnRailAcquireSessionTicket, LocalUserNum, bIsNewAccount);
		Subsystem->QueueAsyncTask(new FOnlineAsyncTaskRailAcquireSessionTicket(Subsystem, RailId, CompletionDelegate));

		UE_LOG_ONLINE_IDENTITY(Log, TEXT("Logged in user %d with token=%s"), LocalUserNum, *UserAccount->AuthToken);
	}

	const bool bWasSuccessful = ErrorStr.IsEmpty();
	if (!bWasSuccessful)
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Login request failed. %s"), *ErrorStr);
		TriggerOnLoginCompleteDelegates(LocalUserNum, bWasSuccessful, *FUniqueNetIdRail::EmptyId(), ErrorStr);
	}

	return bWasSuccessful;
}

void FOnlineIdentityTencent::OnRailAcquireSessionTicket(const FOnlineError& OnlineError, const FString& SessionTicket, int32 LocalUserNum, const bool bIsNewUser)
{
	if (OnlineError.WasSuccessful())
	{
		if (UserAccount.IsValid())
		{
			const FUniqueNetIdRail& UserId = static_cast<const FUniqueNetIdRail&>(*UserAccount->GetUserId());
			UserAccount->AuthToken = FString::Printf(TEXT("%llu:%s:%s"), UserId.RailID.get_id(), *Subsystem->GetAppId(), *SessionTicket);
			TriggerOnLoginCompleteDelegates(LocalUserNum, true, UserId, OnlineError.GetErrorCode());
			if (bIsNewUser)
			{
				TriggerOnLoginStatusChangedDelegates(LocalUserNum, ELoginStatus::NotLoggedIn, ELoginStatus::LoggedIn, UserId);
				TriggerOnLoginChangedDelegates(LocalUserNum);
			}
		}
		else
		{
			UE_LOG_ONLINE_IDENTITY(Error, TEXT("FOnlineIdentityTencent::OnRailAcquireSessionTicket: Missing UserAccount! Did we call logout?"));
		}
	}
	else
	{
		// Do not clear the user data, the user will change based on events from Rail
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("Login request failed (delegate). %s"), *OnlineError.ToLogString());
		TriggerOnLoginCompleteDelegates(LocalUserNum, false, *FUniqueNetIdRail::EmptyId(), OnlineError.GetErrorCode());
	}
}

bool FOnlineIdentityTencent::Logout(int32 LocalUserNum)
{
	UE_LOG_ONLINE_IDENTITY(Log, TEXT("Logout user %d"), LocalUserNum);

	const ELoginStatus::Type LastLoginStatus = GetLoginStatus(LocalUserNum);
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);

	if (UserAccount.IsValid())
	{
		UserAccount.Reset();
		UserIds.Remove(LocalUserNum);
		TriggerOnLogoutCompleteDelegates(LocalUserNum, true);
	}
	else
	{
		UE_LOG_ONLINE_IDENTITY(Warning, TEXT("No logged in user found for LocalUserNum=%d."), LocalUserNum);
		TriggerOnLogoutCompleteDelegates(LocalUserNum, false);
	}

	
	if (LastLoginStatus != ELoginStatus::NotLoggedIn && UserId.IsValid())
	{
		TriggerOnLoginStatusChangedDelegates(LocalUserNum, LastLoginStatus, ELoginStatus::NotLoggedIn, *UserId);
		TriggerOnLoginChangedDelegates(LocalUserNum);
	}

	return true;
}

bool FOnlineIdentityTencent::AutoLogin(int32 LocalUserNum)
{
	return Login(LocalUserNum, FOnlineAccountCredentials());
}

ELoginStatus::Type FOnlineIdentityTencent::GetLoginStatus(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		return GetLoginStatus(*UserId);
	}
	return ELoginStatus::NotLoggedIn;
}

ELoginStatus::Type FOnlineIdentityTencent::GetLoginStatus(const FUniqueNetId& UserId) const
{
	//@todo sz - also check WeGame status

	if (UserAccount.IsValid() && 
		*UserAccount->GetUserId() == UserId && 
		!UserAccount->GetAccessToken().IsEmpty())
	{
		return ELoginStatus::LoggedIn;
	}
	return ELoginStatus::NotLoggedIn;
}

FString FOnlineIdentityTencent::GetPlayerNickname(int32 LocalUserNum) const
{
	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		return GetPlayerNickname(*UserId);
	}
	return InvalidWeGameDisplayName;
}

FString FOnlineIdentityTencent::GetPlayerNickname(const FUniqueNetId& UserId) const
{
	// display name will be cached for users that registered or logged in manually
	TSharedPtr<FUserOnlineAccountTencent> FoundUserAccount = GetUserAccountTencent(UserId);
	if (FoundUserAccount.IsValid())
	{
		FString DisplayName = FoundUserAccount->GetDisplayName();
		if (!DisplayName.IsEmpty())
		{
			return DisplayName;
		}
	}
	return InvalidWeGameDisplayName;
}

FString FOnlineIdentityTencent::GetAuthToken(int32 LocalUserNum) const
{
	FString AuthToken;

	FUniqueNetIdPtr UserId = GetUniquePlayerId(LocalUserNum);
	if (UserId.IsValid())
	{
		TSharedPtr<FUserOnlineAccountTencent> FoundUserAccount = GetUserAccountTencent(*UserId);
		if (FoundUserAccount.IsValid())
		{
			AuthToken = FoundUserAccount->GetAccessToken();
		}
	}

	return AuthToken;
}

FOnlineIdentityTencent::FOnlineIdentityTencent(class FOnlineSubsystemTencent* InSubsystem)
	: Subsystem(InSubsystem)
{	
}

FOnlineIdentityTencent::~FOnlineIdentityTencent()
{
}

FPlatformUserId FOnlineIdentityTencent::GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const
{
	for (int i = 0; i < MAX_LOCAL_PLAYERS; ++i)
	{
		auto CurrentUniqueId = GetUniquePlayerId(i);
		if (CurrentUniqueId.IsValid() && (*CurrentUniqueId == UniqueNetId))
		{
			return GetPlatformUserIdFromLocalUserNum(i);
		}
	}

	return PLATFORMUSERID_NONE;
}

void FOnlineIdentityTencent::GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI)
{
	Delegate.ExecuteIfBound(UserId, Privilege, (uint32)EPrivilegeResults::NoFailures);
}

FString FOnlineIdentityTencent::GetAuthType() const
{
	return TEXT("wegame");
}

void FOnlineIdentityTencent::RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate)
{
	UE_LOG_ONLINE_IDENTITY(Display, TEXT("FOnlineIdentityTencent::RevokeAuthToken not implemented"));
	FUniqueNetIdRef UserIdRef(UserId.AsShared());
	Subsystem->ExecuteNextTick([UserIdRef, Delegate]()
	{
		Delegate.ExecuteIfBound(*UserIdRef, FOnlineError(FString(TEXT("RevokeAuthToken not implemented"))));
	});
}

#endif // WITH_TENCENT_RAIL_SDK
#endif // WITH_TENCENTSDK
