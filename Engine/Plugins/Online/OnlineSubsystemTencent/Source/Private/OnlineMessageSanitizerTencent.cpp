// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineMessageSanitizerTencent.h"
#include "OnlineAsyncTaskManagerTencent.h"
#include "OnlineSubsystemTencent.h"

#if WITH_TENCENT_RAIL_SDK
#include "RailSdkWrapper.h"
#endif

/* Returns a bool indicating whether or not any words were replaced in the InOut string. */
static bool FilterProfanity(FString& InOutString)
{
#if WITH_TENCENT_RAIL_SDK
	// Check RailSdk
	RailSdkWrapper& RailSDK = RailSdkWrapper::Get();
	if (rail::IRailUtils* RailUtils = RailSdkWrapper::Get().RailUtils())
	{		
		// Supposedly we can only send kRailUsersDirtyWordsOnceCheckLimit words under length kRailUsersDirtyWordMaxLength
		// but neither of these constants seem to exist in the API.
		rail::RailDirtyWordsCheckResult DirtyWordResult;
		rail::RailString InRailString;
		ToRailString(InOutString, InRailString);
		rail::RailResult Result = RailUtils->DirtyWordsFilter(InRailString, true, &DirtyWordResult);
		if (Result == rail::RailResult::kSuccess && DirtyWordResult.dirty_type != rail::kRailDirtyWordsTypeNormalAllowWords)
		{
			InOutString = LexToString(DirtyWordResult.replace_string);
			return true;
		}
	}
#endif
	return false;
}

void FSanitizeMessage::Filter()
{
	FilterProfanity(RawMessage);
}

void FSanitizeMessage::TriggerDelegate(bool bSuccess)
{
	CompletionDelegate.ExecuteIfBound(bSuccess, RawMessage);
}

void FSanitizeMessageArray::Filter()
{
	for (FString& RawMessage : RawMessageArray)
	{
		FilterProfanity(RawMessage);
	}
}

void FSanitizeMessageArray::TriggerDelegate(bool bSuccess)
{
	CompletionDelegate.ExecuteIfBound(bSuccess, RawMessageArray);
}

void FMessageSanitizerTask::DoWork()
{
	// Have the message object filter itself.
	Data.Message->Filter();
	Data.bSuccess = true;

	// Queue up an event in the async task manager so that the delegate can safely trigger in the game thread.
	FOnlineAsyncTaskManagerTencent* TaskManager = TencentSubsystem->GetAsyncTaskManager();
	if (TaskManager)
	{
		FAsyncEventSanitizerTaskCompleted* NewEvent = new FAsyncEventSanitizerTaskCompleted(TencentSubsystem, Data);
		TaskManager->AddToOutQueue(NewEvent);
	}
}

void FAsyncEventSanitizerTaskCompleted::TriggerDelegates()
{
	// this must be called from the main thread
	check(IsInGameThread());

	FOnlineAsyncEvent::TriggerDelegates();
	Data.Message->TriggerDelegate(Data.bSuccess);
}

void FMessageSanitizerTencent::SanitizeDisplayName(const FString& DisplayName, const FOnMessageProcessed& CompletionDelegate)
{
	const FString* FoundString = WordMap.Find(DisplayName);
	if (FoundString)
	{
		CompletionDelegate.ExecuteIfBound(true, *FoundString);
	}
	else
	{
		FOnMessageProcessed MessageSanitizerCallback = FOnMessageProcessed::CreateThreadSafeSP(this, &FMessageSanitizerTencent::HandleMessageSanitized, CompletionDelegate, DisplayName);
		TSharedRef<FSanitizeMessage, ESPMode::ThreadSafe> SanitizeMessage = MakeShared<FSanitizeMessage, ESPMode::ThreadSafe>(DisplayName, MessageSanitizerCallback);
		ProcessList.Add(SanitizeMessage);
	}
}

void FMessageSanitizerTencent::SanitizeDisplayNames(const TArray<FString>& DisplayNames, const FOnMessageArrayProcessed& CompletionDelegate)
{
	TArray<FString> AlreadyProcessedMessages;
	TArray<int32> AlreadyProcessedIndex;
	
	TArray<FString> MessagesToProcess;

	for (int32 iMessageIndex = 0; iMessageIndex < DisplayNames.Num(); iMessageIndex++)
	{
		const FString& DisplayName = DisplayNames[iMessageIndex];
		const FString* FoundString = WordMap.Find(DisplayName);
		if (!FoundString)
		{
			MessagesToProcess.Add(*DisplayName);
		}
		else
		{
			AlreadyProcessedMessages.Add(*FoundString);
			AlreadyProcessedIndex.Add(iMessageIndex);
		}
	}

	// If we have messages to process, pack them into a struct for processing.
	if (MessagesToProcess.Num() != 0)
	{
		TSharedRef<FMultiPartMessage> MultiPartMessage = MakeShared<FMultiPartMessage>();
		MultiPartMessage->MessagesToSanitize = MessagesToProcess;
		MultiPartMessage->AlreadyProcessedMessages = MoveTemp(AlreadyProcessedMessages);
		MultiPartMessage->AlreadyProcessedIndex = AlreadyProcessedIndex;
		FOnMessageArrayProcessed MessageSanitizerCallback = FOnMessageArrayProcessed::CreateRaw(this, &FMessageSanitizerTencent::HandleMessageArraySanitized, CompletionDelegate, MultiPartMessage);
		TSharedRef<FSanitizeMessageArray, ESPMode::ThreadSafe> SanitizeMessageArray = MakeShared<FSanitizeMessageArray, ESPMode::ThreadSafe>(MoveTemp(MessagesToProcess), MessageSanitizerCallback);
		ProcessList.Add(SanitizeMessageArray);
	}
	// If we don't have anything to process, it's all been cached
	else
	{
		CompletionDelegate.ExecuteIfBound(true, MoveTemp(AlreadyProcessedMessages));
	}
}

void FMessageSanitizerTencent::QueryBlockedUser(int32 LocalUserNum, const FString& FromUserId, const FString& FromPlatform, const FOnQueryUserBlockedResponse& CompletionDelegate)
{
	// Not supported, always return that user is not blocked.
	FBlockedQueryResult Result;
	Result.UserId = FromUserId;
	Result.bIsBlocked = false;
	CompletionDelegate.ExecuteIfBound(Result);
}

bool FMessageSanitizerTencent::HandleTicker(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FMessageSanitizerTencent_HandleTicker);
	if (ProcessList.Num())
	{
		FSanitizerTaskData Data(ProcessList[0]);
		TSharedRef<MessageSanitizerTask> SanitizerTask = MakeShared<MessageSanitizerTask>(TencentSubsystem, Data);
		SanitizerTask->StartBackgroundTask();
		CurrentTasks.Add(SanitizerTask);
		ProcessList.RemoveAt(0);
	}

	CleanTaskList();
	return true;
}

void FMessageSanitizerTencent::CleanTaskList()
{
	for (int i = CurrentTasks.Num() - 1; i >= 0; --i)
	{
		if (CurrentTasks[i]->IsDone())
		{
			CurrentTasks.RemoveAt(i);
		}
	}
}

void FMessageSanitizerTencent::HandleMessageSanitized(bool bSuccess, const FString& SanitizedMessage, FOnMessageProcessed CompletionDelegate, FString UnsanitizedMessage)
{
	// Add response
	if (!WordMap.Find(UnsanitizedMessage))
	{
		WordMap.Add(UnsanitizedMessage, SanitizedMessage);
	}
	CompletionDelegate.ExecuteIfBound(bSuccess, SanitizedMessage);
}

void FMessageSanitizerTencent::HandleMessageArraySanitized(bool bSuccess,
	const TArray<FString>& SanitizedMessages,
	FOnMessageArrayProcessed CompletionDelegate,
	TSharedRef<FMultiPartMessage> MultiPartMessage)
{

	TArray<FString> SanitizedStrings;
	if (bSuccess == false)
	{
		CompletionDelegate.ExecuteIfBound(false, SanitizedStrings);
	}
	else
	{
		SanitizedStrings = SanitizedMessages;
		
		TArray<FString>& UnsanitizedStrings = MultiPartMessage->MessagesToSanitize;

		check(SanitizedStrings.Num() == UnsanitizedStrings.Num())

			for (int32 iMessageIndex = 0; iMessageIndex < UnsanitizedStrings.Num(); iMessageIndex++)
			{
				if (!WordMap.Find(UnsanitizedStrings[iMessageIndex]))
				{
					WordMap.Add(UnsanitizedStrings[iMessageIndex], SanitizedStrings[iMessageIndex]);
				}
			}

		for (int32 MessageIndex = 0; MessageIndex < MultiPartMessage->AlreadyProcessedMessages.Num(); MessageIndex++)
		{
			SanitizedStrings.Insert(MultiPartMessage->AlreadyProcessedMessages[MessageIndex], MultiPartMessage->AlreadyProcessedIndex[MessageIndex]);
		}

		CompletionDelegate.ExecuteIfBound(true, SanitizedStrings);	
	}
}

FMessageSanitizerTencent::~FMessageSanitizerTencent()
{
	ProcessList.Empty();
	FTSTicker::RemoveTicker(TickDelegateHandle);
}
