// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IMessageSanitizerInterface.h"
#include "Async/AsyncWork.h"
#include "Containers/Ticker.h"
#include "OnlineAsyncTaskManager.h"

class FOnlineSubsystemTencent;

/* Interface for Message sanitization data */
struct ISanitizeMessage
{
	virtual ~ISanitizeMessage() {}

	virtual void Filter() = 0;
	virtual void TriggerDelegate(bool bSuccess) = 0;
};

/* Sanitization data holding a single string to filter */
struct FSanitizeMessage : public ISanitizeMessage
{
	FSanitizeMessage(const FString& InRawMessage, FOnMessageProcessed InProcessCompleteDelegate)
		: CompletionDelegate(InProcessCompleteDelegate)
		, RawMessage(InRawMessage)
	{
	}

	virtual ~FSanitizeMessage(){}

	virtual void Filter() override;
	virtual void TriggerDelegate(bool bSuccess) override;

	FOnMessageProcessed CompletionDelegate;
	FString RawMessage;
};

/* Sanitization data holding an array of strings to filter */
struct FSanitizeMessageArray : public ISanitizeMessage
{
	FSanitizeMessageArray(TArray<FString>&& InRawMessageArray, FOnMessageArrayProcessed InProcessCompleteDelegate)
		: CompletionDelegate(InProcessCompleteDelegate)
		, RawMessageArray(InRawMessageArray)
	{
	}

	virtual ~FSanitizeMessageArray() {}

	virtual void Filter() override;
	virtual void TriggerDelegate(bool bSuccess) override;

	FOnMessageArrayProcessed CompletionDelegate;
	TArray<FString> RawMessageArray;
};

/* Structure used to tie the filtered words back to the original request*/
struct FMultiPartMessage
{
	TArray<FString> AlreadyProcessedMessages;
	TArray<int32> AlreadyProcessedIndex;
	TArray<FString> MessagesToSanitize;
};

struct FSanitizerTaskData
{
	FSanitizerTaskData(TSharedRef<ISanitizeMessage, ESPMode::ThreadSafe> InMessage)
		: Message(InMessage)
		, bSuccess(false)
	{}

	TSharedRef<ISanitizeMessage, ESPMode::ThreadSafe> Message;
	bool bSuccess;
};

// worker task
class FMessageSanitizerTask : public FNonAbandonableTask
{
public:

	/** Constructor */
	FMessageSanitizerTask(FOnlineSubsystemTencent * InTencentSubsystem, const FSanitizerTaskData & InData)
		: TencentSubsystem(InTencentSubsystem), Data(InData)
	{
	}

	/** Performs work on thread */
	void DoWork();

	/** Returns the stat id for this task */
	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FMessageSanitizerTask, STATGROUP_ThreadPoolAsyncTasks);
	}

private:
	FOnlineSubsystemTencent* TencentSubsystem;
	FSanitizerTaskData Data;
};

typedef FAsyncTask<FMessageSanitizerTask> MessageSanitizerTask;

class FAsyncEventSanitizerTaskCompleted : public FOnlineAsyncEvent<FOnlineSubsystemTencent>
{
private:
	FSanitizerTaskData Data;

public:
	/**
	* Constructor.
	*
	* @param InTencentSubsystem The owner of the external UI interface that triggered this event.
	* @param InData All the data relating to the task
	*/
	FAsyncEventSanitizerTaskCompleted(FOnlineSubsystemTencent* InTencentSubsystem, const FSanitizerTaskData & InData) :
		FOnlineAsyncEvent(InTencentSubsystem),
		Data(InData)
	{
	}

	virtual FString ToString() const override
	{
		return TEXT("Sanitize string complete");
	}

	virtual void TriggerDelegates() override;
};


/**
* Implements the Tencent specific interface chat message sanitization
*/
class FMessageSanitizerTencent :
	public IMessageSanitizer
{

public:

	// IMessageSanitizer
	virtual void SanitizeDisplayName(const FString& DisplayName, const FOnMessageProcessed& CompletionDelegate) override;
	virtual void SanitizeDisplayNames(const TArray<FString>& DisplayNames, const FOnMessageArrayProcessed& CompletionDelegate) override;
	virtual void QueryBlockedUser(int32 LocalUserNum, const FString& FromUserId, const FString& FromPlatform, const FOnQueryUserBlockedResponse& CompletionDelegate) override;
	virtual void ResetBlockedUserCache() override {}
	// FMessageSanitizerSwitch

	explicit FMessageSanitizerTencent(FOnlineSubsystemTencent* InTencentSubsystem) :
		TencentSubsystem(InTencentSubsystem)
	{
		check(TencentSubsystem);
		FTickerDelegate TickDelegate = FTickerDelegate::CreateRaw(this, &FMessageSanitizerTencent::HandleTicker);
		TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(TickDelegate, 1.0f);
	}

	virtual ~FMessageSanitizerTencent();

private:
	bool HandleTicker(float DeltaTime);
	void CleanTaskList();
	void HandleMessageSanitized(bool bSuccess, const FString& SanitizedMessage, FOnMessageProcessed CompletionDelegate, FString UnsanitizedMessage);
	void HandleMessageArraySanitized(bool bSuccess, const TArray<FString>& SanitizedMessages,
		FOnMessageArrayProcessed CompletionDelegate,
		TSharedRef<FMultiPartMessage> MultiPartMessage);

private:

	// Holds the list of messages being processed
	TArray<TSharedRef<ISanitizeMessage, ESPMode::ThreadSafe>> ProcessList;

	/** List of current tasks */
	TArray<TSharedPtr<MessageSanitizerTask>> CurrentTasks;

	// Holds a map of sanitized words
	TMap<FString, FString> WordMap;

	/** Handle to the registered TickDelegate. */
	FTSTicker::FDelegateHandle TickDelegateHandle;

	/** Reference to the main Tencent subsystem */
	FOnlineSubsystemTencent* TencentSubsystem;
};

typedef TSharedPtr<FMessageSanitizerTencent, ESPMode::ThreadSafe> FMessageSanitizerTencentPtr;
