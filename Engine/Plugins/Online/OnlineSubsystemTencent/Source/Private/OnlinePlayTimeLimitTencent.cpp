// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlinePlayTimeLimitTencent.h"

#if WITH_TENCENT_RAIL_SDK

#include "OnlineSubsystemTencent.h"
#include "RailSdkWrapper.h"

FOnlinePlayTimeLimitUserTencentRail::~FOnlinePlayTimeLimitUserTencentRail()
{
	if (IOnlineSubsystem::IsLoaded())
	{
		FOnlineSubsystemTencent* OSS = StaticCast<FOnlineSubsystemTencent*>(IOnlineSubsystem::Get(TENCENT_SUBSYSTEM));
		if (OSS)
		{
			OSS->ClearOnAASDialogDelegate_Handle(AASDelegateHandle);
		}
	}
}

void FOnlinePlayTimeLimitUserTencentRail::Init()
{
	FPlayTimeLimitUser::Init();

	FOnlineSubsystemTencent* OSS = StaticCast<FOnlineSubsystemTencent*>(IOnlineSubsystem::Get(TENCENT_SUBSYSTEM));
	if (OSS)
	{
		const FOnAASDialogDelegate& AASDelegate = FOnAASDialogDelegate::CreateThreadSafeSP(this, &FOnlinePlayTimeLimitUserTencentRail::HandleAASDialog);
		AASDelegateHandle = OSS->AddOnAASDialogDelegate_Handle(AASDelegate);
	}
}

bool FOnlinePlayTimeLimitUserTencentRail::HasTimeLimit() const
{
	bool bHasTimeLimit = false;
	rail::IRailPlayer* const RailPlayer = RailSdkWrapper::Get().RailPlayer();
	if (RailPlayer)
	{
		bHasTimeLimit = RailPlayer->IsGameRevenueLimited();
	}
	else
	{
		UE_LOG_ONLINE(Warning, TEXT("HasTimeLimit: Missing RailPlayer"));
	}
	return bHasTimeLimit;
}

int32 FOnlinePlayTimeLimitUserTencentRail::GetPlayTimeMinutes() const
{
	int32 PlayTimeMinutes = 0;
	// Only track number of minutes played for users that have a limit
	if (HasTimeLimit())
	{
		rail::IRailPlayer* const RailPlayer = RailSdkWrapper::Get().RailPlayer();
		check(RailPlayer); // HasTimeLimit should only return true if everything is valid

		// @todo use new API if it becomes available, for now guess based on reward rate
		float RewardRate = RailPlayer->GetRateOfGameRevenue();
		if (FMath::IsNearlyEqual(RewardRate, 1.0f, KINDA_SMALL_NUMBER))
		{
			// Full rewards up to 3 hours
			PlayTimeMinutes = 0;
		}
		else if (FMath::IsNearlyEqual(RewardRate, 0.5f, KINDA_SMALL_NUMBER))
		{
			// Half rewards from 3 hours to 5 hours
			PlayTimeMinutes = 60 * 3;
		}
		else if (FMath::IsNearlyZero(RewardRate, KINDA_SMALL_NUMBER))
		{
			// No rewards after 5 hours
			PlayTimeMinutes = 60 * 5;
		}
		else
		{
			static bool bWarned = false;
			if (!bWarned)
			{
				UE_LOG_ONLINE(Warning, TEXT("GetPlayTimeMinutes: Received RewardRate=%0.4f, unexpected"), RewardRate);
				bWarned = true;
			}
			// Assume 5 hours then.
			PlayTimeMinutes = 60 * 5;
		}
	}
	return PlayTimeMinutes;
}

float FOnlinePlayTimeLimitUserTencentRail::GetRewardRate() const
{
	float RewardRate = 1.0f;
	if (HasTimeLimit())
	{
		rail::IRailPlayer* const RailPlayer = RailSdkWrapper::Get().RailPlayer();
		check(RailPlayer); // HasTimeLimit should only return true if everything is valid

		RewardRate = RailPlayer->GetRateOfGameRevenue();
	}
	return RewardRate;
}

void FOnlinePlayTimeLimitUserTencentRail::HandleAASDialog(const FString& DialogTitle, const FString& DialogText, const FString& ButtonText)
{
	// RailSDK has told us that we should display an anti addiction message.
	// Set time until next message to 0, and let the PlayTimeLimit system handle
	// displaying the message.

	double Now = FPlatformTime::Seconds();
	OverrideDialogTitle = DialogTitle;
	OverrideDialogText = DialogText;
	OverrideButtonText = ButtonText;
	NextNotificationTime = Now;
}

#endif // WITH_TENCENT_RAIL_SDK
