// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if WITH_TENCENTSDK
#if WITH_TENCENT_RAIL_SDK

#include "Delegates/IDelegateInstance.h"
#include "PlayTimeLimitUser.h"

class ONLINESUBSYSTEMTENCENT_API FOnlinePlayTimeLimitUserTencentRail
	: public FPlayTimeLimitUser
{
public:
	explicit FOnlinePlayTimeLimitUserTencentRail(const FUniqueNetIdRef& InUserId)
		: FPlayTimeLimitUser(InUserId)
	{}
	~FOnlinePlayTimeLimitUserTencentRail();

	//~ Begin FPlayTimeLimitUser Interface
	virtual bool HasTimeLimit() const override;
	virtual int32 GetPlayTimeMinutes() const override;
	virtual float GetRewardRate() const override;
	virtual void Init() override;

	// Tencent display times are controlled by Rail SDK events.
	// just clear the existing time if an attempt is made.
	virtual void SetNextNotificationTime(const TOptional<double>& InNextNotificationTime) override { NextNotificationTime = TOptional<double>(); }
	//~ End FPlayTimeLimitUser Interface

	void HandleAASDialog(const FString& DialogTitle, const FString& DialogText, const FString& ButtonText);

	FDelegateHandle AASDelegateHandle;
};

#endif // WITH_TENCENT_RAIL_SDK
#endif // WITH_TENCENTSDK
