// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlinePresenceTencent.h"

#if WITH_TENCENTSDK
#if WITH_TENCENT_RAIL_SDK

#include "OnlineSubsystemTencent.h"
#include "OnlineIdentityTencent.h"
#include "OnlineSessionTencent.h"
#include "MetadataKeysRail.h"

FOnlinePresenceTencent::FOnlinePresenceTencent(FOnlineSubsystemTencent* InSubsystem)
	: TencentSubsystem(InSubsystem)
{ 
}

FOnlinePresenceTencent::~FOnlinePresenceTencent()
{ 
	IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		IdentityInt->ClearOnLoginChangedDelegate_Handle(OnLoginChangedHandle);
	}
}

bool FOnlinePresenceTencent::Init()
{
	IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		OnLoginChangedHandle = IdentityInt->AddOnLoginChangedDelegate_Handle(FOnLoginChangedDelegate::CreateThreadSafeSP(this, &FOnlinePresenceTencent::OnLoginChanged));

		FOnFriendMetadataChangedDelegate Delegate = FOnFriendMetadataChangedDelegate::CreateThreadSafeSP(this, &FOnlinePresenceTencent::OnFriendMetadataChangedEvent);
		OnFriendMetadataChangedDelegateHandle = TencentSubsystem->AddOnFriendMetadataChangedDelegate_Handle(Delegate);

		if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
		{
			// Cache the initial online state of all friends
			rail::RailArray<rail::RailFriendInfo> Friends;
			rail::RailResult Result = RailFriends->GetFriendsList(&Friends);
			if (Result == rail::kSuccess)
			{
				TArray<FUniqueNetIdRef> FriendIds;
				for (uint32 RailIdx = 0; RailIdx < Friends.size(); ++RailIdx)
				{
					const rail::RailFriendInfo& RailFriendInfo(Friends[RailIdx]);
					if (RailFriendInfo.friend_rail_id != rail::kInvalidRailId)
					{
						FUniqueNetIdRef FriendId(FUniqueNetIdRail::Create(RailFriendInfo.friend_rail_id));
						UE_LOG_ONLINE_PRESENCE(VeryVerbose, TEXT("Friend Id: %s State: %s"), *FriendId->ToDebugString(), *LexToString(RailFriendInfo.online_state.friend_online_state));

						SetUserOnlineState(*FriendId, RailOnlineStateToOnlinePresence(RailFriendInfo.online_state.friend_online_state));
					}
					else
					{
						UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Invalid friend in friends list"));
					}
				}
			}
			else
			{
				UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Unable to read friends friends list"));
			}
		}
		else
		{
			UE_LOG_ONLINE_PRESENCE(Warning, TEXT("Rail friends interface is null"));
		}

		return true;
	}

	return false;
}

void FOnlinePresenceTencent::Shutdown()
{
	TencentSubsystem->ClearOnFriendMetadataChangedDelegate_Handle(OnFriendMetadataChangedDelegateHandle);
}

void FOnlinePresenceTencent::SetUserOnlineState(const FUniqueNetId& UserId, EOnlinePresenceState::Type NewState)
{
	const TSharedRef<FOnlineUserPresenceTencent>* Found = CachedPresence.Find(UserId.AsShared());
	if (Found != nullptr)
	{
		(*Found)->bIsOnline = (NewState != EOnlinePresenceState::Type::Offline);
		(*Found)->Status.State = NewState;
	}

	EOnlinePresenceState::Type& OnlineState = UserOnlineStatus.FindOrAdd(UserId.AsShared());
	OnlineState = NewState;
}

void FOnlinePresenceTencent::OnLoginChanged(int32 LocalUserNum)
{
	IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		FUniqueNetIdRailPtr UserId = StaticCastSharedPtr<const FUniqueNetIdRail>(IdentityInt->GetUniquePlayerId(LocalUserNum));
		if (UserId.IsValid())
		{
			ELoginStatus::Type LoginStatus = IdentityInt->GetLoginStatus(LocalUserNum);
			if (LoginStatus == ELoginStatus::NotLoggedIn)
			{
				FOnlineAsyncTaskRailClearAllMetadata* NewTask = new FOnlineAsyncTaskRailClearAllMetadata(TencentSubsystem, *UserId);
				TencentSubsystem->QueueAsyncTask(NewTask);
				int32 NumRemoved = CachedPresence.Remove(UserId.ToSharedRef());
			}
		}
	}
}

void FOnlinePresenceTencent::OnFriendMetadataChangedEvent(const FUniqueNetId& UserId, const FMetadataPropertiesRail& Metadata)
{
	UE_LOG_ONLINE_PRESENCE(Verbose, TEXT("FOnlinePresenceTencent::OnFriendMetadataChangedEvent"));
}

bool IsPresenceEqual(const FOnlineUserPresenceStatus& A, const FOnlineUserPresenceStatus& B)
{
	// Session and joinability details are updated internally and don't need comparison
	const bool bEqual = (A.StatusStr == B.StatusStr) &&
		(A.State == B.State) &&
		(A.Properties.OrderIndependentCompareEqual(B.Properties));
	return bEqual;
}

void FOnlinePresenceTencent::SetPresence(const FUniqueNetId& User, const FOnlineUserPresenceStatus& Status, const FOnPresenceTaskCompleteDelegate& Delegate)
{
	bool bNeedsUpdate = true;
	FUniqueNetIdRef UserPtr = User.AsShared();
	TSharedRef<FOnlineUserPresenceTencent>* Cached = CachedPresence.Find(User.AsShared());
	if (Cached != nullptr)
	{
		bNeedsUpdate = !IsPresenceEqual((*Cached)->Status, Status);
	}
	else
	{
		Cached = &CachedPresence.Add(User.AsShared(), MakeShared<FOnlineUserPresenceTencent>());
	}
	(*Cached)->Status = Status;

	if (bNeedsUpdate)
	{
		FMetadataPropertiesRail Metadata;

		Metadata.Add(RAIL_PRESENCE_STATUS_KEY, Status.StatusStr);
		Metadata.Add(RAIL_PRESENCE_APPID_KEY, TencentSubsystem->GetAppId());
		Metadata.Append(Status.Properties);

		GetGameSessionPresenceData(Metadata);

		FOnlineAsyncTaskRailSetUserMetadata* NewTask = new FOnlineAsyncTaskRailSetUserPresence(TencentSubsystem, Metadata, FOnOnlineAsyncTaskRailSetUserMetadataComplete::CreateLambda([Delegate, UserPtr](const FSetUserMetadataTaskResult& Result)
		{
			Delegate.ExecuteIfBound(*UserPtr, Result.Error.WasSuccessful());
		}));

		TencentSubsystem->QueueAsyncTask(NewTask);
	}
	else
	{
		UE_LOG_ONLINE_PRESENCE(Verbose, TEXT("SetPresence data already set."));
		TencentSubsystem->ExecuteNextTick([UserPtr, Delegate]()
		{
			Delegate.ExecuteIfBound(*UserPtr, true);
		});
	}
}

void FOnlinePresenceTencent::GetGameSessionPresenceData(FMetadataPropertiesRail& InMetadata)
{
	// Get the user's session, if any
	FString UserSessionId;
	
	bool bIsOnline = true;
	bool bIsPlaying = false;
	bool bIsJoinable = false;
	bool bHasVoiceSupport = false;

	FOnlineSessionTencentPtr SessionInt = StaticCastSharedPtr<FOnlineSessionTencent>(TencentSubsystem->GetSessionInterface());
	if (SessionInt.IsValid())
	{
		bool bPublicJoinable = false;
		bool bFriendJoinable = false;
		bool bInviteOnly = false;
		bool bAllowInvites = false;
		FNamedOnlineSession* PresenceSession = SessionInt->GetPresenceSession();
		if (PresenceSession != nullptr &&
			PresenceSession->GetJoinability(bPublicJoinable, bFriendJoinable, bInviteOnly, bAllowInvites))
		{
			TSharedPtr<FOnlineSessionInfo> UserSessionInfo = PresenceSession->SessionInfo;
			if (UserSessionInfo.IsValid())
			{
				bIsPlaying = true;
				UserSessionId = UserSessionInfo->GetSessionId().ToString();
				bIsJoinable = (bPublicJoinable || bFriendJoinable) && (!bInviteOnly && bAllowInvites);
			}
		}
	}

	int32 NumBits = 0;
	uint32 PresenceBits = 0;
	PresenceBits |= bIsOnline ? (1 << NumBits) : 0; NumBits++;
	PresenceBits |= bIsPlaying ? (1 << NumBits) : 0; NumBits++;
	PresenceBits |= bIsJoinable ? (1 << NumBits) : 0; NumBits++;
	PresenceBits |= bHasVoiceSupport ? (1 << NumBits) : 0; NumBits++;

	InMetadata.Add(RAIL_PRESENCE_PRESENCEBITS_KEY, FVariantData(PresenceBits));
	InMetadata.Add(RAIL_PRESENCE_SESSION_ID_KEY, FVariantData(UserSessionId));
}

void FOnlinePresenceTencent::UpdatePresenceFromSessionData()
{
	FMetadataPropertiesRail Metadata;
	GetGameSessionPresenceData(Metadata);

	FOnlineAsyncTaskRailSetUserMetadata* NewTask = new FOnlineAsyncTaskRailSetUserMetadata(TencentSubsystem, Metadata, FOnOnlineAsyncTaskRailSetUserMetadataComplete::CreateLambda([](const FSetUserMetadataTaskResult& Result)
	{
		UE_LOG_ONLINE_PRESENCE(Verbose, TEXT("UpdatePresenceFromSessionData %s"), *Result.Error.ToLogString());
	}));

	TencentSubsystem->QueueAsyncTask(NewTask);
}

void FOnlinePresenceTencent::QueryPresence(const FUniqueNetId& User, const FOnPresenceTaskCompleteDelegate& Delegate)
{
	FUniqueNetIdRef UserPtr = User.AsShared();
	TSharedRef<FOnlinePresenceTencent, ESPMode::ThreadSafe> StrongThis = AsShared();
	FOnlineAsyncTaskRailGetUserPresence* NewTask = new FOnlineAsyncTaskRailGetUserPresence(TencentSubsystem, (const FUniqueNetIdRail&)User, FOnOnlineAsyncTaskRailGetUserMetadataComplete::CreateLambda([StrongThis, Delegate, UserPtr](const FGetUserMetadataTaskResult& InResult)
	{
		if (InResult.Error.WasSuccessful() && InResult.UserId.IsValid())
		{
			const TSharedRef<FOnlineUserPresenceTencent>* Cached = StrongThis->CachedPresence.Find(InResult.UserId.ToSharedRef());
			if (Cached == nullptr)
			{
				Cached = &StrongThis->CachedPresence.Add(InResult.UserId.ToSharedRef(), MakeShared<FOnlineUserPresenceTencent>());
			}

			const FVariantData* PresenceBitsData = InResult.Metadata.Find(RAIL_PRESENCE_PRESENCEBITS_KEY);
			if (PresenceBitsData && PresenceBitsData->GetType() == EOnlineKeyValuePairDataType::UInt32)
			{
				uint32 PresenceBits;
				PresenceBitsData->GetValue(PresenceBits);

				int32 NumBits = 0;
				(*Cached)->bIsOnline = !!(PresenceBits & (1 << NumBits)); NumBits++;
				(*Cached)->bIsPlaying = !!(PresenceBits & (1 << NumBits)); NumBits++;
				(*Cached)->bIsJoinable = !!(PresenceBits & (1 << NumBits)); NumBits++;
				(*Cached)->bHasVoiceSupport = !!(PresenceBits & (1 << NumBits)); NumBits++;
			}
			else
			{
				(*Cached)->bIsOnline = false;
				(*Cached)->bIsPlaying = false;
				(*Cached)->bIsJoinable = false;
				(*Cached)->bHasVoiceSupport = false;
			}

			const FVariantData* PresenceSessionIdData = InResult.Metadata.Find(RAIL_PRESENCE_SESSION_ID_KEY);
			if (PresenceSessionIdData && PresenceSessionIdData->GetType() == EOnlineKeyValuePairDataType::String)
			{
				FString SessionIdStr;
				PresenceSessionIdData->GetValue(SessionIdStr);
				if (!SessionIdStr.IsEmpty())
				{
					FOnlineSessionTencentPtr SessionInt = StaticCastSharedPtr<FOnlineSessionTencent>(StrongThis->TencentSubsystem->GetSessionInterface());
					if (SessionInt.IsValid())
					{
						(*Cached)->SessionId = SessionInt->CreateSessionIdFromString(SessionIdStr);
					}
				}
			}
			else
			{
				(*Cached)->SessionId = FUniqueNetIdRail::EmptyId();
			}

			const FVariantData* AppIdData = InResult.Metadata.Find(RAIL_PRESENCE_APPID_KEY);
			if (AppIdData && AppIdData->GetType() == EOnlineKeyValuePairDataType::String)
			{
				(*Cached)->bIsPlayingThisGame = (StrongThis->TencentSubsystem->GetAppId() == AppIdData->ToString());
			}
			else
			{
				(*Cached)->bIsPlayingThisGame = false;
			}

			const FVariantData* StatusData = InResult.Metadata.Find(RAIL_PRESENCE_STATUS_KEY);
			if (StatusData && StatusData->GetType() == EOnlineKeyValuePairDataType::String)
			{
				StatusData->GetValue((*Cached)->Status.StatusStr);
			}
			else
			{
				(*Cached)->Status.StatusStr.Empty();
			}

			EOnlinePresenceState::Type OnlineState = EOnlinePresenceState::Offline;
			if (const EOnlinePresenceState::Type* UserOnlineState = StrongThis->UserOnlineStatus.Find(InResult.UserId.ToSharedRef()))
			{
				// Override the presence data with latest state from Rail status
				OnlineState = *UserOnlineState;
				(*Cached)->bIsOnline = (OnlineState != EOnlinePresenceState::Offline);
			}
			else if (PresenceBitsData)
			{
				// No cached data but presence key data
				OnlineState = (*Cached)->bIsOnline ? EOnlinePresenceState::Online : EOnlinePresenceState::Offline;
			}
			else
			{
				// Neither pieces of information has to mean the user is offline
				(*Cached)->bIsOnline = false;
			}

			(*Cached)->Status.State = OnlineState;
			(*Cached)->Status.Properties = InResult.Metadata;
		}
		Delegate.ExecuteIfBound(*UserPtr, InResult.Error.WasSuccessful());
	}));

	TencentSubsystem->QueueAsyncTask(NewTask);
}

EOnlineCachedResult::Type FOnlinePresenceTencent::GetCachedPresence(const FUniqueNetId& User, TSharedPtr<FOnlineUserPresence>& OutPresence)
{
	EOnlineCachedResult::Type Result = EOnlineCachedResult::NotFound;

	const TSharedRef<FOnlineUserPresenceTencent>* Found = CachedPresence.Find(User.AsShared());
	if (Found != nullptr)
	{
		Result = EOnlineCachedResult::Success;
		OutPresence = *Found;
	}

	return Result;
}

EOnlineCachedResult::Type FOnlinePresenceTencent::GetCachedPresenceForApp(const FUniqueNetId& LocalUserId, const FUniqueNetId& User, const FString& AppId, TSharedPtr<FOnlineUserPresence>& OutPresence)
{
	/** NYI */
	return EOnlineCachedResult::Type::NotFound;
}

#endif // WITH_TENCENTSDK
#endif // WITH_TENCENT_RAIL_SDK
