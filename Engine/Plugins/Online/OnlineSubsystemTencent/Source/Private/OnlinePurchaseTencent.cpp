// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlinePurchaseTencent.h"

#if WITH_TENCENT_RAIL_SDK

#include "OnlineSubsystemTencent.h"
#include "OnlineError.h"
#include "OnlineAsyncTasksTencent.h"
#include "Interfaces/OnlineIdentityInterface.h"

FOnlinePurchaseTencent::FOnlinePurchaseTencent(FOnlineSubsystemTencent* const InSubsystem)
	: Subsystem(InSubsystem)
{
	check(Subsystem != nullptr);
}

bool FOnlinePurchaseTencent::IsAllowedToPurchase(const FUniqueNetId& UserId)
{
	const bool bResult = true;
	return bResult;
}

bool FOnlinePurchaseTencent::IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error)
{
	Error = FOnlineError(EOnlineErrorResult::Success);
	return true;
}

void FOnlinePurchaseTencent::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate)
{
	FOnlineError Result(true);
	
	// Validate user
	{
		const IOnlineIdentityPtr IdentityInt = Subsystem->GetIdentityInterface();
		if (IdentityInt.IsValid())
		{
			ELoginStatus::Type LoginStatus = IdentityInt->GetLoginStatus(UserId);
			if (LoginStatus != ELoginStatus::LoggedIn)
			{
				Result.SetFromErrorCode(TEXT("com.epicgames.purchase.invalid_user"));
			}
		}
		else
		{
			Result.SetFromErrorCode(TEXT("com.epicgames.purchase.missing_identity"));
		}
		Result.bSucceeded = Result.ErrorCode.IsEmpty();
	}

	// Validate request
	if (Result.WasSuccessful())
	{
		if (CheckoutRequest.PurchaseOffers.Num() <= 0)
		{
			Result.SetFromErrorCode(TEXT("com.epicgames.purchase.no_offers"));
		}
		Result.bSucceeded = Result.ErrorCode.IsEmpty();
	}

	// Only allow one request at a time
	if (Result.WasSuccessful())
	{
		if (bCheckoutPending)
		{
			Result.SetFromErrorCode(TEXT("com.epicgames.purchase.already_in_checkout"));
		}
		Result.bSucceeded = Result.ErrorCode.IsEmpty();
	}

	if (Result.WasSuccessful())
	{
		bCheckoutPending = true;
		FOnOnlineAsyncTaskRailPurchaseProductsComplete CompletionDelegate;
		CompletionDelegate.BindThreadSafeSP(this, &FOnlinePurchaseTencent::Checkout_Complete, Delegate);

		FOnlineAsyncTaskRailPurchaseProducts* AsyncTask = new FOnlineAsyncTaskRailPurchaseProducts(Subsystem, UserId.AsShared(), CheckoutRequest, CompletionDelegate);
		Subsystem->QueueAsyncTask(AsyncTask);
	}

	if (!Result.WasSuccessful())
	{
		UE_LOG_ONLINE_PURCHASE(Log, TEXT("Checkout: Failed with Result=[%s]"), *Result.ToLogString());
		Subsystem->ExecuteNextTick([Result = MoveTemp(Result), Delegate]()
		{
			static const TSharedRef<FPurchaseReceipt> BlankReceipt(MakeShared<FPurchaseReceipt>());
			Delegate.ExecuteIfBound(Result, BlankReceipt);
		});
	}
}

void FOnlinePurchaseTencent::Checkout_Complete(const FPurchaseProductsTaskResult& Result, FOnPurchaseCheckoutComplete Delegate)
{
	UE_LOG_ONLINE_PURCHASE(Log, TEXT("Checkout_Complete: [%s]"), *Result.Error.ToLogString());
	bCheckoutPending = false;
	static TSharedRef<FPurchaseReceipt> EmptyReceipt = MakeShared<FPurchaseReceipt>();
	Delegate.ExecuteIfBound(Result.Error, Result.Error.WasSuccessful() ? Result.PurchaseReceipt.ToSharedRef() : EmptyReceipt);
}

void FOnlinePurchaseTencent::Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate)
{
	// NYI - please use the other Checkout method
	Subsystem->ExecuteNextTick([Delegate]()
	{
		static const FOnlineError NotImplementedResult(EOnlineErrorResult::NotImplemented);
		Delegate.ExecuteIfBound(NotImplementedResult);
	});
}

void FOnlinePurchaseTencent::FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId)
{
	// NYI
}

void FOnlinePurchaseTencent::RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate)
{
	// NYI
	Subsystem->ExecuteNextTick([Delegate]()
	{
		static const FOnlineError NotImplementedResult(TEXT("not implemented"));
		static const TSharedRef<FPurchaseReceipt> BlankReceipt(MakeShared<FPurchaseReceipt>());
		Delegate.ExecuteIfBound(NotImplementedResult, BlankReceipt);
	});
}

void FOnlinePurchaseTencent::QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate)
{
	const FUniqueNetIdRef UserIdRef(UserId.AsShared());
	FOnOnlineAsyncTaskRailRequestAllAssetsComplete CompletionDelegate;
	CompletionDelegate.BindThreadSafeSP(this, &FOnlinePurchaseTencent::QueryReceipts_Complete, UserIdRef, Delegate);

	FOnlineAsyncTaskRailRequestAllAssets* AsyncTask = new FOnlineAsyncTaskRailRequestAllAssets(Subsystem, UserIdRef, CompletionDelegate);
	Subsystem->QueueAsyncTask(AsyncTask);
}

void FOnlinePurchaseTencent::QueryReceipts_Complete(const FRequestAllAssetsTaskResult& Result, const FUniqueNetIdRef UserId, const FOnQueryReceiptsComplete Delegate)
{
	UE_LOG_ONLINE_PURCHASE(Verbose, TEXT("QueryReceipts Complete with result=[%s]"), *Result.Error.ToLogString());
	PurchaseReceipts.Empty(1);
	if (Result.Error.WasSuccessful())
	{
		PurchaseReceipts.Emplace(Result.PurchaseReceipt);
	}
	Delegate.ExecuteIfBound(Result.Error);
}

void FOnlinePurchaseTencent::GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const
{
	OutReceipts = PurchaseReceipts;
}

void FOnlinePurchaseTencent::OnRailAssetsChanged(const FUniqueNetId& UserId)
{
	// Broadcast that an unexpected purchase receipt was received
	if (!bCheckoutPending)
	{
		TriggerOnUnexpectedPurchaseReceiptDelegates(UserId);
	}
}

void FOnlinePurchaseTencent::FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate)
{
	FOnlineError DefaultSuccess(true);
	Delegate.ExecuteIfBound(DefaultSuccess, InReceiptValidationInfo);
}
#endif // WITH_TENCENT_RAIL_SDK
