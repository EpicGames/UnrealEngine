// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TENCENT_RAIL_SDK

#include "Interfaces/OnlinePurchaseInterface.h"
#include "OnlineSubsystemTencentPackage.h"

class FOnlineSubsystemTencent;
struct FRequestAllAssetsTaskResult;
struct FPurchaseProductsTaskResult;

/**
 * Implementation of IOnlinePurchase using the Rail SDK
 */
class FOnlinePurchaseTencent
	: public IOnlinePurchase
	, public TSharedFromThis<FOnlinePurchaseTencent, ESPMode::ThreadSafe>
{
public:
	/** Destructor */
	virtual ~FOnlinePurchaseTencent() = default;

	// Begin IOnlinePurchase
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId) override;
	virtual bool IsAllowedToPurchase(const FUniqueNetId& UserId, FOnlineError& Error) override;
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseCheckoutComplete& Delegate) override;
	virtual void Checkout(const FUniqueNetId& UserId, const FPurchaseCheckoutRequest& CheckoutRequest, const FOnPurchaseReceiptlessCheckoutComplete& Delegate) override;
	virtual void FinalizePurchase(const FUniqueNetId& UserId, const FString& ReceiptId) override;
	virtual void RedeemCode(const FUniqueNetId& UserId, const FRedeemCodeRequest& RedeemCodeRequest, const FOnPurchaseRedeemCodeComplete& Delegate) override;
	virtual void QueryReceipts(const FUniqueNetId& UserId, bool bRestoreReceipts, const FOnQueryReceiptsComplete& Delegate) override;
	virtual void GetReceipts(const FUniqueNetId& UserId, TArray<FPurchaseReceipt>& OutReceipts) const override;
	virtual void FinalizeReceiptValidationInfo(const FUniqueNetId& UserId, FString& InReceiptValidationInfo, const FOnFinalizeReceiptValidationInfoComplete& Delegate) override;
	// End IOnlinePurchase

PACKAGE_SCOPE:
	/**
	 * Constructor
	 * @param InSubsystem the owning subsystem
	 */
	FOnlinePurchaseTencent(FOnlineSubsystemTencent* const InSubsystem);

	/**
	 * Called via an event emitted from the Rail SDK when a user's assets have changed
	 */
	void OnRailAssetsChanged(const FUniqueNetId& UserId);

private:

	/** Called when the QueryReceipts async task completes */
	void QueryReceipts_Complete(const FRequestAllAssetsTaskResult& Result, const FUniqueNetIdRef UserId, const FOnQueryReceiptsComplete Delegate);

	/** Called when the Checkout async task completes */
	void Checkout_Complete(const FPurchaseProductsTaskResult& Result, FOnPurchaseCheckoutComplete Delegate);

	/** Default constructor disabled */
	FOnlinePurchaseTencent() = delete;
	
	/** Owning subsystem */
	FOnlineSubsystemTencent* const Subsystem;

	/** Purchase receipts */
	TArray<FPurchaseReceipt> PurchaseReceipts;

	/** Only allow one checkout at a time */
	bool bCheckoutPending = false;
};

typedef TSharedPtr<FOnlinePurchaseTencent, ESPMode::ThreadSafe> FOnlinePurchaseTencentPtr;

#endif // WITH_TENCENT_RAIL_SDK
