// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSessionTencent.h"
#include "OnlineSubsystemTencent.h"
#include "OnlineIdentityTencent.h"
#include "Misc/ScopeLock.h"

#if WITH_TENCENTSDK

FOnlineSessionInfoTencent::FOnlineSessionInfoTencent() 
	: SessionId(nullptr)
{
}

FOnlineSessionInfoTencent::FOnlineSessionInfoTencent(const FUniqueNetIdStringPtr& InSessionId)
	: SessionId(InSessionId)
{
}

void FOnlineSessionInfoTencent::Init()
{
	FGuid NewGuid = FGuid::NewGuid();
	SessionId = FUniqueNetIdString::Create(NewGuid.ToString(), TENCENT_SUBSYSTEM);
}

FUniqueNetIdPtr FOnlineSessionTencent::CreateSessionIdFromString(const FString& SessionIdStr)
{
	FUniqueNetIdPtr SessionId;
	if (!SessionIdStr.IsEmpty())
	{
		SessionId = FUniqueNetIdString::Create(SessionIdStr, TENCENT_SUBSYSTEM);
	}
	return SessionId;
}

FNamedOnlineSession* FOnlineSessionTencent::AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings)
{
	FScopeLock ScopeLock(&SessionLock);
	return new (Sessions) FNamedOnlineSessionTencent(SessionName, SessionSettings);
}

FNamedOnlineSession* FOnlineSessionTencent::AddNamedSession(FName SessionName, const FOnlineSession& Session)
{
	FScopeLock ScopeLock(&SessionLock);
	return new (Sessions)FNamedOnlineSessionTencent(SessionName, Session);
}

FNamedOnlineSession* FOnlineSessionTencent::GetNamedSession(FName SessionName)
{
	FScopeLock ScopeLock(&SessionLock);
	for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
	{
		if (Sessions[SearchIndex].SessionName == SessionName)
		{
			return &Sessions[SearchIndex];
		}
	}
	return nullptr;
}

void FOnlineSessionTencent::RemoveNamedSession(FName SessionName)
{
	FScopeLock ScopeLock(&SessionLock);
	for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
	{
		if (Sessions[SearchIndex].SessionName == SessionName)
		{
			Sessions.RemoveAtSwap(SearchIndex);
			return;
		}
	}
}

EOnlineSessionState::Type FOnlineSessionTencent::GetSessionState(FName SessionName) const
{
	FScopeLock ScopeLock(&SessionLock);
	for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
	{
		if (Sessions[SearchIndex].SessionName == SessionName)
		{
			return Sessions[SearchIndex].SessionState;
		}
	}

	return EOnlineSessionState::NoSession;
}

bool FOnlineSessionTencent::HasPresenceSession()
{
	FScopeLock ScopeLock(&SessionLock);
	for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
	{
		if (Sessions[SearchIndex].SessionSettings.bUsesPresence)
		{
			return true;
		}
	}
	
	return false;
}

bool FOnlineSessionTencent::IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId)
{ 
	return IsPlayerInSessionImpl(this, SessionName, UniqueId);
}

FOnlineSessionSettings* FOnlineSessionTencent::GetSessionSettings(FName SessionName)
{
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		return &Session->SessionSettings;
	}
	return nullptr;
}

int32 FOnlineSessionTencent::GetNumSessions()
{
	FScopeLock ScopeLock(&SessionLock);
	return Sessions.Num();
}

void FOnlineSessionTencent::DumpSessionState()
{
	FScopeLock ScopeLock(&SessionLock);

	for (int32 SessionIdx = 0; SessionIdx < Sessions.Num(); SessionIdx++)
	{
		DumpNamedSession(&Sessions[SessionIdx]);
	}
}

FNamedOnlineSession* FOnlineSessionTencent::GetPresenceSession()
{
	FScopeLock ScopeLock(&SessionLock);
	for (int32 SearchIndex = 0; SearchIndex < Sessions.Num(); SearchIndex++)
	{
		FNamedOnlineSession& Session = Sessions[SearchIndex];
		if (Session.SessionSettings.bUsesPresence)
		{
			return &Sessions[SearchIndex];
		}
	}
	return nullptr;
}

bool FOnlineSessionTencent::StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings) { return false; }
bool FOnlineSessionTencent::CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName) { return false; }
bool FOnlineSessionTencent::CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName) { return false; }
bool FOnlineSessionTencent::FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings) { return false; }
bool FOnlineSessionTencent::FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings) { return false; }
bool FOnlineSessionTencent::FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate)
{
	FOnlineSessionSearchResult EmptyResult;
	const int32 LocalUserNum = TencentSubsystem->GetIdentityInterface()->GetPlatformUserIdFromUniqueNetId(SearchingUserId);
	CompletionDelegate.ExecuteIfBound(LocalUserNum, false, EmptyResult);
	return false;
}
bool FOnlineSessionTencent::CancelFindSessions() { return false; }
bool FOnlineSessionTencent::PingSearchResults(const FOnlineSessionSearchResult& SearchResult) { return false; }
bool FOnlineSessionTencent::FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend) { return false; }
bool FOnlineSessionTencent::FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend) { return false; }
bool FOnlineSessionTencent::FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList) { return false; }
bool FOnlineSessionTencent::SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend) { return false; }
bool FOnlineSessionTencent::SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend) { return false; }
bool FOnlineSessionTencent::SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) { return false; }
bool FOnlineSessionTencent::SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) { return false; }
bool FOnlineSessionTencent::GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType) { return false; }
bool FOnlineSessionTencent::GetResolvedConnectString(const FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo) { return false; }

int32 FOnlineSessionTencent::GetLocalUserIdx(const FUniqueNetId& UserId) const
{
	FOnlineIdentityTencentPtr IdentityInt = StaticCastSharedPtr<FOnlineIdentityTencent>(TencentSubsystem->GetIdentityInterface());
	if (IdentityInt)
	{
		int32 LocalUserNum = INDEX_NONE;
		if (IdentityInt->GetLocalUserIdx(UserId, LocalUserNum))
		{
			return LocalUserNum;
		}
	}
	return INDEX_NONE;
}

#endif // WITH_TENCENTSDK
