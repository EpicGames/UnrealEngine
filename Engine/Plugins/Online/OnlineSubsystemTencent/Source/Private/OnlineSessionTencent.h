// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSessionSettings.h"
#include "OnlineSubsystemTencentPackage.h"

#if WITH_TENCENTSDK

class FOnlineSubsystemTencent;

class FNamedOnlineSessionTencent : public FNamedOnlineSession
{
private:

	FNamedOnlineSessionTencent() = delete;

public:
	/** Constructor used to create a named session directly */
	FNamedOnlineSessionTencent(FName InSessionName, const FOnlineSessionSettings& InSessionSettings)
		: FNamedOnlineSession(InSessionName, InSessionSettings)
	{
	}

	/** Constructor used to create a named session directly */
	FNamedOnlineSessionTencent(FName InSessionName, const FOnlineSession& Session)
		: FNamedOnlineSession(InSessionName, Session)
	{
	}

	virtual ~FNamedOnlineSessionTencent()
	{
	}
};

/**
 * 
 */
class FOnlineSessionTencent : public IOnlineSession, public TSharedFromThis<FOnlineSessionTencent, ESPMode::ThreadSafe>
{
private:
	/** Hidden on purpose */
	FOnlineSessionTencent() = delete;

public:

	// ~Begin IOnlineSession Interface
	virtual FUniqueNetIdPtr CreateSessionIdFromString(const FString& SessionIdStr) override;
	virtual class FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSessionSettings& SessionSettings) override;
	virtual class FNamedOnlineSession* AddNamedSession(FName SessionName, const FOnlineSession& Session) override;
	virtual FNamedOnlineSession* GetNamedSession(FName SessionName) override;
	virtual void RemoveNamedSession(FName SessionName) override;
	virtual EOnlineSessionState::Type GetSessionState(FName SessionName) const override;
	virtual bool HasPresenceSession() override;
	virtual bool IsPlayerInSession(FName SessionName, const FUniqueNetId& UniqueId) override;
	virtual bool StartMatchmaking(const TArray< FUniqueNetIdRef >& LocalPlayers, FName SessionName, const FOnlineSessionSettings& NewSessionSettings, TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool CancelMatchmaking(int32 SearchingPlayerNum, FName SessionName) override;
	virtual bool CancelMatchmaking(const FUniqueNetId& SearchingPlayerId, FName SessionName) override;
	virtual bool FindSessions(int32 SearchingPlayerNum, const TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool FindSessions(const FUniqueNetId& SearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& SearchSettings) override;
	virtual bool FindSessionById(const FUniqueNetId& SearchingUserId, const FUniqueNetId& SessionId, const FUniqueNetId& FriendId, const FOnSingleSessionResultCompleteDelegate& CompletionDelegate) override;
	virtual bool CancelFindSessions() override;
	virtual bool PingSearchResults(const FOnlineSessionSearchResult& SearchResult) override;
	virtual bool FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend) override;
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend) override;
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList) override;
	virtual bool SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend) override;
	virtual bool SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend) override;
	virtual bool SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;
	virtual bool SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;
	virtual bool GetResolvedConnectString(FName SessionName, FString& ConnectInfo, FName PortType) override;
	virtual bool GetResolvedConnectString(const class FOnlineSessionSearchResult& SearchResult, FName PortType, FString& ConnectInfo) override;
	virtual FOnlineSessionSettings* GetSessionSettings(FName SessionName) override;
	virtual int32 GetNumSessions() override;
	virtual void DumpSessionState() override;
	// ~End IOnlineSession Interface

PACKAGE_SCOPE:

	FOnlineSubsystemTencent* TencentSubsystem;

	FOnlineSessionTencent(FOnlineSubsystemTencent* InSubsystem) :
		TencentSubsystem(InSubsystem)
	{
	}

	virtual ~FOnlineSessionTencent() {}

	/**
	 * Initialize the session interface
	 */
	virtual bool Init() { return true; }

	/**
	 * Session tick for various background tasks
	 */
	virtual void Tick(float DeltaTime) {}

	/**
	 * Shutdown the session interface
	 */
	virtual void Shutdown() {}

	/**
	 * @return the one session marked as having presence about a user
	 */
	FNamedOnlineSession* GetPresenceSession();

protected:

	FNamedOnlineSessionTencent* GetNamedSessionTencent(FName SessionName)
	{
		return static_cast<FNamedOnlineSessionTencent*>(GetNamedSession(SessionName));
	}

	int32 GetLocalUserIdx(const FUniqueNetId& UserId) const;

	/** Critical sections for thread safe operation of session lists */
	mutable FCriticalSection SessionLock;

	/** Current session settings */
	TArray<FNamedOnlineSessionTencent> Sessions;

};

typedef TSharedPtr<FOnlineSessionTencent, ESPMode::ThreadSafe> FOnlineSessionTencentPtr;

#endif //WITH_TENCENTSDK
