// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSessionTencentRail.h"

#if WITH_TENCENT_RAIL_SDK
#include "Interfaces/OnlineIdentityInterface.h"
#include "Online/OnlineBase.h"
#include "OnlineAsyncTasksTencent.h"
#include "OnlinePresenceTencent.h"
#include "OnlineSubsystemTencent.h"
#include "OnlineSubsystemTencentPrivate.h"
#include "MetadataKeysRail.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"

#define RAIL_INVITE_RAILID TEXT("--rail_connect_to_railid=")
#define RAIL_INVITE_CMDLINE TEXT("--rail_connect_cmd=")

FOnlineSessionTencentRail::FOnlineSessionTencentRail(FOnlineSubsystemTencent* InSubsystem)
	: FOnlineSessionTencent(InSubsystem)
{
	CheckPendingSessionInvite();
}

FOnlineSessionTencentRail::~FOnlineSessionTencentRail()
{
}

bool FOnlineSessionTencentRail::Init()
{
	bool bSuccess = false;
	if (FOnlineSessionTencent::Init())
	{
		FOnFriendMetadataChangedDelegate Delegate = FOnFriendMetadataChangedDelegate::CreateThreadSafeSP(this, &FOnlineSessionTencentRail::OnFriendMetadataChangedEvent);
		OnFriendMetadataChangedDelegateHandle = TencentSubsystem->AddOnFriendMetadataChangedDelegate_Handle(Delegate);
		bSuccess = true;
	}

	return bSuccess;
}

void FOnlineSessionTencentRail::Shutdown()
{
	RailSdkWrapper& RailSDK = RailSdkWrapper::Get();
	if (RailSDK.IsInitialized())
	{
		rail::IRailFactory* RailFactory = RailSDK.RailFactory();
		if (RailFactory)
		{
			rail::IRailFriends* Friends = RailFactory->RailFriends();
			if (Friends)
			{
				rail::RailResult RailResult = Friends->AsyncSetInviteCommandLine(rail::RailString(), rail::RailString());
			}
		}
	}

	TencentSubsystem->ClearOnFriendMetadataChangedDelegate_Handle(OnFriendMetadataChangedDelegateHandle);
}

void FOnlineSessionTencentRail::Tick(float DeltaTime)
{
	//SCOPE_CYCLE_COUNTER(STAT_Session_Interface);
	TickPendingInvites(DeltaTime);
}

void FOnlineSessionTencentRail::OnFriendMetadataChangedEvent(const FUniqueNetId& UserId, const FMetadataPropertiesRail& Metadata)
{
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("FOnlineSessionTencentRail::OnFriendMetadataChangedEvent"));
}

void FOnlineSessionTencentRail::CheckPendingSessionInvite()
{
	const TCHAR* CmdLine = FCommandLine::Get();
	FString CmdLineStr(CmdLine);

	const FString UserRailIdCmd = RAIL_INVITE_RAILID;
	int32 UserIdIdx = CmdLineStr.Find(UserRailIdCmd, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	if (UserIdIdx != INDEX_NONE)
	{
		// Go to the value for the parameter
		const TCHAR* Str = CmdLine + UserIdIdx + UserRailIdCmd.Len();
		FString UserIdIdStr = FParse::Token(Str, 0).TrimStartAndEnd();
		int64 UserId = FCString::Strtoui64(*UserIdIdStr, NULL, 10);
		if (UserId > 0)
		{
			PendingInvite.InviterUserId = FUniqueNetIdRail::Create(UserId);
			PendingInvite.bValidInvite = true;
		}
	}
	
	const FString UserCmdLineArgs = RAIL_INVITE_CMDLINE;
	int32 CmdLineIdx = CmdLineStr.Find(UserCmdLineArgs, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	if (CmdLineIdx != INDEX_NONE)
	{
		// Go to the value for the parameter
		const TCHAR* Str = CmdLine + CmdLineIdx + UserCmdLineArgs.Len();
		FString UserCmdLineStr = FParse::Token(Str, 0).TrimStartAndEnd();
		if (!UserCmdLineStr.IsEmpty())
		{
			PendingInvite.CommandLineArgs = UserCmdLineStr;
		}
	}
}

void FOnlineSessionTencentRail::TickPendingInvites(float DeltaTime)
{
	if (PendingInvite.bValidInvite)
	{
		if (OnSessionUserInviteAcceptedDelegates.IsBound())
		{
			IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();

			// Wait until we have a valid user
			FUniqueNetIdRailPtr UserId = StaticCastSharedPtr<const FUniqueNetIdRail>(GetFirstSignedInUser(IdentityInt));
			if (UserId.IsValid() && ensure(PendingInvite.InviterUserId.IsValid()))
			{
				QueryAcceptedUserInvitation(UserId.ToSharedRef(), PendingInvite.InviterUserId.ToSharedRef());
			
				// Clear the invite
				PendingInvite.bValidInvite = false;
			}
		}
	}
}

void FOnlineSessionTencentRail::QueryAcceptedUserInvitation(FUniqueNetIdRailRef InLocalUser, FUniqueNetIdRailRef InRemoteUser)
{
	TWeakPtr<FOnlineSessionTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
	FOnOnlineAsyncTaskRailGetUserInviteComplete CompletionDelegate = FOnOnlineAsyncTaskRailGetUserInviteComplete::CreateLambda([InLocalUser, LocalWeakThis](const FGetUserInviteTaskResult& Result)
	{
		FOnlineSessionTencentRailPtr StrongThis = StaticCastSharedPtr<FOnlineSessionTencentRail>(LocalWeakThis.Pin());
		if (StrongThis.IsValid())
		{
			const int32 LocalUserIdx = StrongThis->GetLocalUserIdx(*InLocalUser);
			if (Result.Error.WasSuccessful() && Result.Metadata.Num() && !Result.Commandline.IsEmpty())
			{
				TSharedRef<FOnlineSessionSearch> SessionSearch = MakeShared<FOnlineSessionSearch>();
				StrongThis->ParseSearchResult(SessionSearch, Result);
				if (SessionSearch->SearchResults.Num())
				{
					StrongThis->TriggerOnSessionUserInviteAcceptedDelegates(Result.Error.WasSuccessful(), LocalUserIdx, InLocalUser, SessionSearch->SearchResults[0]);
				}
				else
				{
					FOnlineSessionSearchResult EmptyResult;
					StrongThis->TriggerOnSessionUserInviteAcceptedDelegates(Result.Error.WasSuccessful(), LocalUserIdx, InLocalUser, EmptyResult);
				}
			}
			else
			{
				FOnlineSessionSearchResult EmptyResult;
				StrongThis->TriggerOnSessionUserInviteAcceptedDelegates(Result.Error.WasSuccessful(), LocalUserIdx, InLocalUser, EmptyResult);
			}
		}
	});

	FOnlineAsyncTaskRailGetUserInvite* NewTask = new FOnlineAsyncTaskRailGetUserInvite(TencentSubsystem, *InRemoteUser, CompletionDelegate);
	UE_LOG_ONLINE_SESSION(Verbose, TEXT("%s"), *NewTask->ToString());
	TencentSubsystem->QueueAsyncTask(NewTask);
}

bool FOnlineSessionTencentRail::CreateSession(int32 HostingPlayerNum, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	uint32 Result = ONLINE_FAIL;

	// Check for an existing session
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session == nullptr)
	{
		IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
		if (IdentityInt.IsValid() && IdentityInt->GetLoginStatus(HostingPlayerNum) >= ELoginStatus::UsingLocalProfile)
		{
			// Create a new session and deep copy the game settings
			Session = AddNamedSession(SessionName, NewSessionSettings);
			check(Session);
			Session->SessionState = EOnlineSessionState::Creating;

			Session->OwningUserId = IdentityInt->GetUniquePlayerId(HostingPlayerNum);
			Session->OwningUserName = IdentityInt->GetPlayerNickname(HostingPlayerNum);
			
			if (Session->OwningUserId.IsValid() && Session->OwningUserId->IsValid())
			{
				// RegisterPlayer will update these values for the local player
				Session->NumOpenPrivateConnections = NewSessionSettings.NumPrivateConnections;
				Session->NumOpenPublicConnections = NewSessionSettings.NumPublicConnections;

				Session->HostingPlayerNum = HostingPlayerNum;

				// Unique identifier of this build for compatibility
				Session->SessionSettings.BuildUniqueId = GetBuildUniqueId();

				// Create Internet or LAN match
				if (!NewSessionSettings.bIsLANMatch)
				{
					Result = CreateInternetSession(HostingPlayerNum, Session);
				}
				else
				{
					/** LAN NYI */
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot create session '%s': invalid user (%d)."), *SessionName.ToString(), HostingPlayerNum);
			}

			if (Result != ONLINE_IO_PENDING)
			{
				// Set the game state as pending (not started)
				Session->SessionState = EOnlineSessionState::Pending;

				if (Result != ONLINE_SUCCESS)
				{
					// Clean up the session info so we don't get into a confused state
					RemoveNamedSession(SessionName);
				}
				else
				{
					RegisterLocalPlayers(Session);
				}
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot create session '%s': user not logged in (%d)."), *SessionName.ToString(), HostingPlayerNum);
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Cannot create session '%s': session already exists."), *SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		TWeakPtr<FOnlineSessionTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
		TencentSubsystem->ExecuteNextTick([LocalWeakThis, SessionName, Result]()
		{
			FOnlineSessionTencentRailPtr StrongThis = StaticCastSharedPtr<FOnlineSessionTencentRail>(LocalWeakThis.Pin());
			if (StrongThis.IsValid())
			{
				StrongThis->TriggerOnCreateSessionCompleteDelegates(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
			}
		});	
	}

	return (Result == ONLINE_IO_PENDING) || (Result == ONLINE_SUCCESS);
}

bool FOnlineSessionTencentRail::CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings)
{
	return CreateSession(GetLocalUserIdx(HostingPlayerId), SessionName, NewSessionSettings);
}

uint32 FOnlineSessionTencentRail::CreateInternetSession(int32 HostingPlayerNum, FNamedOnlineSession* Session)
{
	check(Session && !Session->SessionInfo.IsValid());
	TSharedRef<FOnlineSessionInfoTencent> SessionInfo = MakeShared<FOnlineSessionInfoTencent>();
	SessionInfo->Init();
	Session->SessionInfo = SessionInfo;

	if (Session->SessionSettings.bUsesPresence)
	{
		// Always at least one frame later
		FName SessionName = Session->SessionName;
		TWeakPtr<FOnlineSessionTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
		UpdateSessionMetadata(*Session, FOnUpdateSessionMetadataComplete::CreateLambda([SessionName, LocalWeakThis](const FOnlineError& Error)
		{
			FOnlineSessionTencentRailPtr StrongThis = StaticCastSharedPtr<FOnlineSessionTencentRail>(LocalWeakThis.Pin());
			if (StrongThis.IsValid())
			{
				StrongThis->OnCreateInternetSessionComplete(SessionName, Error);
			}
		}));

		return SessionInfo->IsValid() ? ONLINE_IO_PENDING : ONLINE_FAIL;
	}
	else
	{
		return SessionInfo->IsValid() ? ONLINE_SUCCESS : ONLINE_FAIL;
	}
}

void FOnlineSessionTencentRail::OnCreateInternetSessionComplete(FName SessionName, const FOnlineError& Error)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("CreateSession %s: %s"), *SessionName.ToString(), *Error.ToLogString());
	bool bWasSuccessful = Error.WasSuccessful();
	FNamedOnlineSession* NamedSession = GetNamedSession(SessionName);
	if (NamedSession)
	{
		if (Error.WasSuccessful())
		{
			NamedSession->SessionState = EOnlineSessionState::Pending;
		}
		else
		{
			RemoveNamedSession(SessionName);
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to find session during creation = %s"), *SessionName.ToString());
		bWasSuccessful = false;
	}

	TriggerOnCreateSessionCompleteDelegates(SessionName, bWasSuccessful);
}

bool FOnlineSessionTencentRail::StartSession(FName SessionName)
{
	uint32 Result = ONLINE_FAIL;
	// Grab the session information by name
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		// Can't start a match multiple times
		if (Session->SessionState == EOnlineSessionState::Pending ||
			Session->SessionState == EOnlineSessionState::Ended)
		{
			if (!Session->SessionSettings.bIsLANMatch)
			{
				Result = StartInternetSession(Session);
			}
			else
			{
				/** NYI LAN */
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't start an online session (%s) in state %s"),
				*SessionName.ToString(),
				EOnlineSessionState::ToString(Session->SessionState));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't start an online game for session (%s) that hasn't been created"), *SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		// Just trigger the delegate
		TriggerOnStartSessionCompleteDelegates(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
	}

	return Result == ONLINE_SUCCESS || Result == ONLINE_IO_PENDING;
}

uint32 FOnlineSessionTencentRail::StartInternetSession(FNamedOnlineSession* Session)
{
	Session->SessionState = EOnlineSessionState::InProgress;
	return ONLINE_SUCCESS;
}

bool FOnlineSessionTencentRail::UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData)
{
	int32 Result = ONLINE_FAIL;

	// Grab the session information by name
	FNamedOnlineSessionTencent* Session = GetNamedSessionTencent(SessionName);
	if (Session)
	{
		if (!Session->SessionSettings.bIsLANMatch)
		{
			TSharedPtr<FOnlineSessionInfoTencent> SessionInfo = StaticCastSharedPtr<FOnlineSessionInfoTencent>(Session->SessionInfo);

			Session->SessionSettings = UpdatedSessionSettings;

			if (bShouldRefreshOnlineData)
			{
				//const bool bOwnsSession = OwnsSession(Session);
				//if (Session->SessionSettings.bUsesPresence && bOwnsSession)
				if (Session->SessionSettings.bUsesPresence)
				{
					TWeakPtr<FOnlineSessionTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
					UpdateSessionMetadata(*Session, FOnUpdateSessionMetadataComplete::CreateLambda([SessionName, LocalWeakThis](const FOnlineError& Error)
					{
						FOnlineSessionTencentRailPtr StrongThis = StaticCastSharedPtr<FOnlineSessionTencentRail>(LocalWeakThis.Pin());
						if (StrongThis.IsValid())
						{
							StrongThis->OnUpdateSessionComplete(SessionName, Error);
						}
					}));
					Result = ONLINE_IO_PENDING;
				}
				else
				{
					Result = ONLINE_SUCCESS;
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Log, TEXT("UpdateInternetSession complete, skipping online refresh."));
				Result = ONLINE_SUCCESS;
			}
		}
		else
		{
			Session->SessionSettings = UpdatedSessionSettings;
			Result = ONLINE_SUCCESS;
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No session (%s) found for update!"), *SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		TriggerOnUpdateSessionCompleteDelegates(SessionName, Result == ONLINE_SUCCESS);
	}

	return Result == ONLINE_SUCCESS || Result == ONLINE_IO_PENDING;
}

void FOnlineSessionTencentRail::OnUpdateSessionComplete(FName SessionName, const FOnlineError& Error)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("UpdateSession %s: %s"), *SessionName.ToString(), *Error.ToLogString());

	bool bWasSuccessful = Error.WasSuccessful();
	FNamedOnlineSession* NamedSession = GetNamedSession(SessionName);
	if (!NamedSession)
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to find session during update = %s"), *SessionName.ToString());
		bWasSuccessful = false;
	}

	TriggerOnUpdateSessionCompleteDelegates(SessionName, bWasSuccessful);
}

bool FOnlineSessionTencentRail::EndSession(FName SessionName)
{
	uint32 Result = ONLINE_FAIL;

	// Grab the session information by name
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		// Can't end a match that isn't in progress
		if (Session->SessionState == EOnlineSessionState::InProgress)
		{
			if (!Session->SessionSettings.bIsLANMatch)
			{
				Result = EndInternetSession(Session);
			}
			else
			{
				/** NYI LAN */
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't end session (%s) in state %s"),
				*SessionName.ToString(),
				EOnlineSessionState::ToString(Session->SessionState));
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't end an online game for session (%s) that hasn't been created"),
			*SessionName.ToString());
	}

	if (Result != ONLINE_IO_PENDING)
	{
		if (Session)
		{
			Session->SessionState = EOnlineSessionState::Ended;
		}

		TriggerOnEndSessionCompleteDelegates(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
	}

	return Result == ONLINE_SUCCESS || Result == ONLINE_IO_PENDING;
}

uint32 FOnlineSessionTencentRail::EndInternetSession(FNamedOnlineSession* Session)
{
	uint32 Result = ONLINE_SUCCESS;

	// Only called from EndSession/DestroySession and presumes only in InProgress state
	check(Session && Session->SessionState == EOnlineSessionState::InProgress);

	Session->SessionState = EOnlineSessionState::Ended;
	return Result;
}

bool FOnlineSessionTencentRail::DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	uint32 Result = ONLINE_FAIL;

	// Find the session in question
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	if (Session)
	{
		if (Session->SessionState != EOnlineSessionState::Destroying)
		{
			if (!Session->SessionSettings.bIsLANMatch)
			{
				if (Session->SessionState == EOnlineSessionState::InProgress)
				{
					// Enqueue all the end session tasks first
					Result = EndInternetSession(Session);
				}

				Result = DestroyInternetSession(Session, CompletionDelegate);
			}
			else
			{
				/** NYI LAN */
			}

			if (Result != ONLINE_IO_PENDING)
			{
				// The session info is no longer needed
				RemoveNamedSession(Session->SessionName);
				CompletionDelegate.ExecuteIfBound(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
				TriggerOnDestroySessionCompleteDelegates(SessionName, (Result == ONLINE_SUCCESS) ? true : false);
			}
		}
		else
		{
			// Purposefully skip the delegate call as one should already be in flight
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Already in process of destroying session (%s)"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Can't destroy a null online session (%s)"), *SessionName.ToString());
		CompletionDelegate.ExecuteIfBound(SessionName, false);
		TriggerOnDestroySessionCompleteDelegates(SessionName, false);
	}

	return Result == ONLINE_SUCCESS || Result == ONLINE_IO_PENDING;
}

uint32 FOnlineSessionTencentRail::DestroyInternetSession(FNamedOnlineSession* Session, const FOnDestroySessionCompleteDelegate& CompletionDelegate)
{
	uint32 Result = ONLINE_SUCCESS;
	Session->SessionState = EOnlineSessionState::Destroying;

	if (Session->SessionSettings.bUsesPresence)
	{
		const FName SessionName = Session->SessionName;

		// Clear the invite command line
		FOnlineAsyncTaskRailSetInviteCommandline* NewCmdLineTask = new FOnlineAsyncTaskRailSetInviteCommandline(TencentSubsystem, FString(), FOnOnlineAsyncTaskRailSetInviteCommandlineComplete());
		TencentSubsystem->QueueAsyncTask(NewCmdLineTask);

		// Clear the session metadata keys by setting all known keys back to empty
		FMetadataPropertiesRail EmptySessionMetadata;
		TWeakPtr<FOnlineSessionTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
		FOnlineAsyncTaskRailSetUserMetadata* NewMetadataTask = new FOnlineAsyncTaskRailSetSessionMetadata(TencentSubsystem, EmptySessionMetadata, FOnOnlineAsyncTaskRailSetUserMetadataComplete::CreateLambda([SessionName, LocalWeakThis, CompletionDelegate](const FSetUserMetadataTaskResult& Result)
		{
			FOnlineSessionTencentRailPtr StrongThis = StaticCastSharedPtr<FOnlineSessionTencentRail>(LocalWeakThis.Pin());
			if (StrongThis.IsValid())
			{
				// The session info is no longer needed
				StrongThis->RemoveNamedSession(SessionName);
				CompletionDelegate.ExecuteIfBound(SessionName, Result.Error.WasSuccessful());
				StrongThis->TriggerOnDestroySessionCompleteDelegates(SessionName, Result.Error.WasSuccessful());
			}
		}));
		TencentSubsystem->QueueAsyncTask(NewMetadataTask);
		Result = ONLINE_IO_PENDING;
	}
	return Result;
}

bool FOnlineSessionTencentRail::JoinSession(int32 PlayerNum, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	uint32 Return = ONLINE_FAIL;
	FNamedOnlineSession* Session = GetNamedSession(SessionName);
	// Don't join a session if already in one or hosting one
	if (Session == nullptr)
	{
		// Create a named session from the search result data
		Session = AddNamedSession(SessionName, DesiredSession.Session);
		Session->HostingPlayerNum = PlayerNum;

		// Create Internet or LAN match
		if (!Session->SessionSettings.bIsLANMatch)
		{
			if (DesiredSession.Session.SessionInfo.IsValid())
			{
				TSharedPtr<const FOnlineSessionInfoTencent> SearchSessionInfo = StaticCastSharedPtr<const FOnlineSessionInfoTencent>(DesiredSession.Session.SessionInfo);

				FOnlineSessionInfoTencent* NewSessionInfo = new FOnlineSessionInfoTencent(SearchSessionInfo->SessionId);
				Session->SessionInfo = MakeShareable(NewSessionInfo);
				Return = JoinInternetSession(PlayerNum, Session, &DesiredSession.Session);
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("Invalid session info on search result"), *SessionName.ToString());
			}
		}
		else
		{
			/** NYI LAN */
		}

		if (Return != ONLINE_IO_PENDING)
		{
			if (Return != ONLINE_SUCCESS)
			{
				// Clean up the session info so we don't get into a confused state
				RemoveNamedSession(SessionName);
			}
			else
			{
				RegisterLocalPlayers(Session);
			}
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Session (%s) already exists, can't join twice"), *SessionName.ToString());
	}

	if (Return != ONLINE_IO_PENDING)
	{
		// Just trigger the delegate as having failed
		TriggerOnJoinSessionCompleteDelegates(SessionName, Return == ONLINE_SUCCESS ? EOnJoinSessionCompleteResult::Success : EOnJoinSessionCompleteResult::UnknownError);
	}

	return Return == ONLINE_SUCCESS || Return == ONLINE_IO_PENDING;
}

bool FOnlineSessionTencentRail::JoinSession(const FUniqueNetId& PlayerId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession)
{
	return JoinSession(GetLocalUserIdx(PlayerId), SessionName, DesiredSession);
}

uint32 FOnlineSessionTencentRail::JoinInternetSession(int32 PlayerNum, FNamedOnlineSession* Session, const FOnlineSession* SearchSession)
{
	uint32 Result = ONLINE_FAIL;
	if (Session->SessionInfo.IsValid())
	{
		TSharedPtr<FOnlineSessionInfoTencent> TencentSessionInfo = StaticCastSharedPtr<FOnlineSessionInfoTencent>(Session->SessionInfo);
		if (TencentSessionInfo->SessionId.IsValid())
		{
			IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
			if (IdentityInt.IsValid())
			{
				FUniqueNetIdPtr UniqueId = IdentityInt->GetUniquePlayerId(PlayerNum);
				if (UniqueId.IsValid() && UniqueId->IsValid())
				{
					if (Session->SessionSettings.bUsesPresence)
					{
						FName SessionName = Session->SessionName;
						TWeakPtr<FOnlineSessionTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
						UpdateSessionMetadata(*Session, FOnUpdateSessionMetadataComplete::CreateLambda([SessionName, LocalWeakThis](const FOnlineError& Error)
						{
							FOnlineSessionTencentRailPtr StrongThis = StaticCastSharedPtr<FOnlineSessionTencentRail>(LocalWeakThis.Pin());
							if (StrongThis.IsValid())
							{
								StrongThis->OnJoinInternetSessionComplete(SessionName, Error);
							}
						}));
						Result = ONLINE_IO_PENDING;
					}
					else
					{
						Session->SessionState = EOnlineSessionState::Pending;
						RegisterLocalPlayers(Session);
						Result = ONLINE_SUCCESS;
					}
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("Session (%s) invalid user id (%d)"), *Session->SessionName.ToString(), PlayerNum);
				}
			}
			else
			{
				UE_LOG_ONLINE_SESSION(Warning, TEXT("No identity interface trying to join session (%s)"), *Session->SessionName.ToString());
			}
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("Session (%s) has invalid session id"), *Session->SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Session (%s) has invalid session info"), *Session->SessionName.ToString());
	}

	return Result;
}

void FOnlineSessionTencentRail::OnJoinInternetSessionComplete(FName SessionName, const FOnlineError& Error)
{
	UE_LOG_ONLINE_SESSION(Warning, TEXT("JoinSession %s: %s"), *SessionName.ToString(), *Error.ToLogString());
	bool bWasSuccessful = Error.WasSuccessful();
	FNamedOnlineSession* NamedSession = GetNamedSession(SessionName);
	if (NamedSession)
	{
		if (Error.WasSuccessful())
		{
			NamedSession->SessionState = EOnlineSessionState::Pending;
		}
		else
		{
			RemoveNamedSession(SessionName);
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("Unable to find session during join = %s"), *SessionName.ToString());
		bWasSuccessful = false;
	}

	TriggerOnJoinSessionCompleteDelegates(SessionName, bWasSuccessful ? EOnJoinSessionCompleteResult::Success : EOnJoinSessionCompleteResult::UnknownError);
}

bool FOnlineSessionTencentRail::RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited)
{
	TArray< FUniqueNetIdRef > Players;
	Players.Add(FUniqueNetIdRail::Create(PlayerId));
	return RegisterPlayers(SessionName, Players, bWasInvited);
}

bool FOnlineSessionTencentRail::RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited)
{
	bool bSuccess = false;

	FNamedOnlineSessionTencent* Session = GetNamedSessionTencent(SessionName);
	if (Session)
	{
		if (Session->SessionInfo.IsValid())
		{
			TArray<FReportPlayedWithUser> ReportedUsers;
			for (int32 PlayerIdx = 0; PlayerIdx < Players.Num(); PlayerIdx++)
			{
				const FUniqueNetIdRef& PlayerId = Players[PlayerIdx];

				FUniqueNetIdMatcher PlayerMatch(*PlayerId);
				if (Session->RegisteredPlayers.IndexOfByPredicate(PlayerMatch) == INDEX_NONE)
				{
					Session->RegisteredPlayers.Add(PlayerId);
					if (GetLocalUserIdx(*PlayerId) == INDEX_NONE)
					{
						// Only report remote users
						ReportedUsers.Emplace(PlayerId, FApp::GetProjectName());
					}
				}
				else
				{

					UE_LOG_ONLINE_SESSION(Log, TEXT("Player %s already registered in session %s"), *PlayerId->ToDebugString(), *SessionName.ToString());
				}
			}

			if (ReportedUsers.Num() > 0)
			{
				FOnlineAsyncTaskRailReportPlayedWithUsers* NewTask = new FOnlineAsyncTaskRailReportPlayedWithUsers(TencentSubsystem, ReportedUsers, FOnOnlineAsyncTaskRailReportPlayedWithUsersComplete::CreateLambda([](const FReportPlayedWithUsersTaskResult& Result)
				{
					if (!Result.Error.WasSuccessful())
					{
						UE_LOG_ONLINE_SESSION(Warning, TEXT("Failed to report player: %s"), *Result.Error.ToLogString());
					}
				}));
				TencentSubsystem->QueueAsyncTask(NewTask);
			}

			bSuccess = true;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("No session info to join for session (%s)"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No game present to join for session (%s)"), *SessionName.ToString());
	}

	TriggerOnRegisterPlayersCompleteDelegates(SessionName, Players, bSuccess);
	return bSuccess;
}

void FOnlineSessionTencentRail::RegisterLocalPlayers(FNamedOnlineSession* Session)
{
	if (!TencentSubsystem->IsDedicated())
	{
		IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
		if (IdentityInt.IsValid())
		{
			TArray<FUniqueNetIdRef > PlayersToRegister;
			for (int32 Index = 0; Index < MAX_LOCAL_PLAYERS; Index++)
			{
				FUniqueNetIdPtr UserId = IdentityInt->GetUniquePlayerId(Index);
				if (UserId.IsValid())
				{
					PlayersToRegister.Add(UserId.ToSharedRef());
				}
			}

			if (PlayersToRegister.Num())
			{
				RegisterPlayers(Session->SessionName, PlayersToRegister, false);
			}
		}
	}
}

bool FOnlineSessionTencentRail::UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId)
{
	TArray< FUniqueNetIdRef > Players;
	Players.Add(FUniqueNetIdRail::Create(PlayerId));
	return UnregisterPlayers(SessionName, Players);
}

bool FOnlineSessionTencentRail::UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players)
{
	bool bSuccess = false;

	FNamedOnlineSessionTencent* Session = GetNamedSessionTencent(SessionName);
	if (Session)
	{
		if (Session->SessionInfo.IsValid())
		{
			for (int32 PlayerIdx = 0; PlayerIdx < Players.Num(); PlayerIdx++)
			{
				const FUniqueNetIdRef& PlayerId = Players[PlayerIdx];

				FUniqueNetIdMatcher PlayerMatch(*PlayerId);
				int32 RegistrantIndex = Session->RegisteredPlayers.IndexOfByPredicate(PlayerMatch);
				if (RegistrantIndex != INDEX_NONE)
				{
					Session->RegisteredPlayers.RemoveAtSwap(RegistrantIndex);
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Warning, TEXT("Player %s is not part of session (%s)"), *PlayerId->ToDebugString(), *SessionName.ToString());
				}
			}

			bSuccess = true;
		}
		else
		{
			UE_LOG_ONLINE_SESSION(Warning, TEXT("No session info to leave for session (%s)"), *SessionName.ToString());
		}
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Warning, TEXT("No game present to leave for session (%s)"), *SessionName.ToString());
	}

	TriggerOnUnregisterPlayersCompleteDelegates(SessionName, Players, bSuccess);
	return bSuccess;
}

bool FOnlineSessionTencentRail::OwnsSession(FNamedOnlineSession* Session) const
{
	bool bOwnsSession = false;

	if (Session)
	{
		IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
		if (IdentityInt.IsValid())
		{
			FUniqueNetIdPtr UserId = IdentityInt->GetUniquePlayerId(Session->HostingPlayerNum);
			bOwnsSession = (UserId.IsValid() && (*UserId == *Session->OwningUserId)) ? true : false;
		}
	}

	return bOwnsSession;
}

void FOnlineSessionTencentRail::RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate)
{
	Delegate.ExecuteIfBound(PlayerId, EOnJoinSessionCompleteResult::Success);
}

void FOnlineSessionTencentRail::UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate)
{
	Delegate.ExecuteIfBound(PlayerId, true);
}

void FOnlineSessionTencentRail::DumpSessionState()
{
	FOnlineSessionTencent::DumpSessionState();
	UE_LOG_ONLINE_SESSION(Display, TEXT("Current Known Session Keys:"));
	for (const FString& CurrentKey : CurrentSessionPresenceKeys)
	{
		UE_LOG_ONLINE_SESSION(Display, TEXT("- %s"), *CurrentKey);
	}
}

void FOnlineSessionTencentRail::UpdateSessionMetadata(const FNamedOnlineSession& InNamedSession, const FOnUpdateSessionMetadataComplete& InCompletionDelegate)
{
	// Does this change require talking to the presence system to update keys
	const bool bIsPresenceSession = InNamedSession.SessionSettings.bUsesPresence;
	// Set/Unset the invite command line based on the session ability to allow invites
	const bool bAllowInvites = InNamedSession.SessionSettings.bAllowInvites;

	FMetadataPropertiesRail NewSessionMetadata;
	if (InNamedSession.SessionInfo.IsValid() && InNamedSession.SessionInfo->IsValid())
	{
		const FOnlineSessionSettings& SessionSettings = InNamedSession.SessionSettings;
		TSharedPtr<const FOnlineSessionInfo> SessionInfo = InNamedSession.SessionInfo;

		NewSessionMetadata.Add(RAIL_SESSION_ID_KEY, FVariantData(InNamedSession.SessionInfo->GetSessionId().ToString()));
		NewSessionMetadata.Add(RAIL_SESSION_OWNING_USER_ID_KEY, FVariantData(InNamedSession.OwningUserId->ToString()));

		int32 NumBits = 0;
		uint32 SessionBits = 0;
		SessionBits |= SessionSettings.bShouldAdvertise ? (1 << NumBits) : 0; NumBits++;
		SessionBits |= SessionSettings.bAllowJoinInProgress ? (1 << NumBits) : 0; NumBits++;
		SessionBits |= SessionSettings.bAllowInvites ? (1 << NumBits) : 0; NumBits++;
		SessionBits |= SessionSettings.bAllowJoinViaPresence ? (1 << NumBits) : 0; NumBits++;
		SessionBits |= SessionSettings.bAllowJoinViaPresenceFriendsOnly ? (1 << NumBits) : 0; NumBits++;

		NewSessionMetadata.Add(RAIL_SESSION_SESSIONBITS_KEY, FVariantData(SessionBits));
		NewSessionMetadata.Add(RAIL_SESSION_BUILDUNIQUEID_KEY, FVariantData(SessionSettings.BuildUniqueId));

		for (const TPair<FName, FOnlineSessionSetting>& Pair : SessionSettings.Settings)
		{
			if ((Pair.Value.AdvertisementType == EOnlineDataAdvertisementType::Type::ViaOnlineService) ||
				(Pair.Value.AdvertisementType == EOnlineDataAdvertisementType::Type::ViaOnlineServiceAndPing))
			{
				NewSessionMetadata.Add(Pair.Key.ToString(), Pair.Value.Data);
			}
		}

		for (const TPair<FString, FVariantData>& Pair : NewSessionMetadata)
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Session Presence Data: [%s] %s"), *Pair.Key, *Pair.Value.ToString());
		}
	}
	else
	{ 
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("Clearing session presence keys, session not advertised"));
	}

	TWeakPtr<FOnlineSessionTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
	FOnlineAsyncTaskRailSetSessionMetadata* NewPresenceTask = new FOnlineAsyncTaskRailSetSessionMetadata(TencentSubsystem, NewSessionMetadata, FOnOnlineAsyncTaskRailSetUserMetadataComplete::CreateLambda([LocalWeakThis, bIsPresenceSession, bAllowInvites, InCompletionDelegate](const FSetUserMetadataTaskResult& Result)
	{
		bool bSecondTaskTriggered = false;
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("UpdateSessionMetadata [Presence] %s"), *Result.Error.ToLogString());
		if (Result.Error.WasSuccessful())
		{
			FOnlineSessionTencentRailPtr StrongThis = StaticCastSharedPtr<FOnlineSessionTencentRail>(LocalWeakThis.Pin());
			if (StrongThis.IsValid())
			{
				// Update the invite command line with the new array of active keys
				FString InviteCmdLine;
				if (bAllowInvites)
				{
					InviteCmdLine = FString::Join(StrongThis->CurrentSessionPresenceKeys, RAIL_METADATA_KEY_SEPARATOR);
				}

				FOnlineAsyncTaskRailSetInviteCommandline* NewCmdlineTask = new FOnlineAsyncTaskRailSetInviteCommandline(StrongThis->TencentSubsystem, InviteCmdLine, FOnOnlineAsyncTaskRailSetInviteCommandlineComplete::CreateLambda([LocalWeakThis, bIsPresenceSession, InCompletionDelegate](const FSetUserMetadataTaskResult& Result)
				{
					UE_LOG_ONLINE_SESSION(Verbose, TEXT("UpdateSessionMetadata [CmdLine] %s"), *Result.Error.ToLogString());
					if (Result.Error.WasSuccessful())
					{
						if (bIsPresenceSession)
						{
							FOnlineSessionTencentRailPtr StrongThis = StaticCastSharedPtr<FOnlineSessionTencentRail>(LocalWeakThis.Pin());
							if (StrongThis.IsValid())
							{
								FOnlinePresenceTencentPtr PresenceInt = StaticCastSharedPtr<FOnlinePresenceTencent>(StrongThis->TencentSubsystem->GetPresenceInterface());
								if (PresenceInt.IsValid())
								{
									// Update presence now that session data has changed
									PresenceInt->UpdatePresenceFromSessionData();
								}
							}
						}
					}

					// Always trigger that UpdateMetadata is complete
					InCompletionDelegate.ExecuteIfBound(Result.Error);
				}));

				StrongThis->TencentSubsystem->QueueAsyncTask(NewCmdlineTask);
				bSecondTaskTriggered = true;
			}
		}

		if (!bSecondTaskTriggered)
		{
			// Always trigger that UpdateMetadata is complete
			InCompletionDelegate.ExecuteIfBound(Result.Error);
		}
	}));

	TencentSubsystem->QueueAsyncTask(NewPresenceTask);
}

void FOnlineSessionTencentRail::ParseSearchResult(TSharedPtr<FOnlineSessionSearch> InSearch, const FGetUserInviteTaskResult& InResult)
{
	FOnlineSessionSearchResult* NewSearchResult = new (InSearch->SearchResults) FOnlineSessionSearchResult();
	FOnlineSessionInfoTencent* SessionInfo = new FOnlineSessionInfoTencent();
	NewSearchResult->Session.SessionInfo = MakeShareable(SessionInfo);

	// Populate the session settings with any known defaults before parsing the result
	NewSearchResult->Session.SessionSettings = *InSearch->GetDefaultSessionSettings();

	const FVariantData* SessionId = InResult.Metadata.Find(RAIL_SESSION_ID_KEY);
	const FVariantData* OwningUserUniqueId = InResult.Metadata.Find(RAIL_SESSION_OWNING_USER_ID_KEY);
	const FVariantData* SessionBitsData = InResult.Metadata.Find(RAIL_SESSION_SESSIONBITS_KEY);
	const FVariantData* RemoteBuildUniqueId = InResult.Metadata.Find(RAIL_SESSION_BUILDUNIQUEID_KEY);
	
	if (SessionId && OwningUserUniqueId && SessionBitsData && RemoteBuildUniqueId)
	{
		FOnlineSessionSettings& SessionSettings = NewSearchResult->Session.SessionSettings;
		SessionSettings.bUsesPresence = true;

		if (SessionId->GetType() == EOnlineKeyValuePairDataType::String)
		{
			FString SessionIdStr;
			SessionId->GetValue(SessionIdStr);
			SessionInfo->SessionId = FUniqueNetIdString::Create(SessionIdStr, TENCENT_SUBSYSTEM);
		}

		if (OwningUserUniqueId->GetType() == EOnlineKeyValuePairDataType::String)
		{
			FString OwningUserUniqueIdStr;
			OwningUserUniqueId->GetValue(OwningUserUniqueIdStr);
			NewSearchResult->Session.OwningUserId = FUniqueNetIdRail::Create(OwningUserUniqueIdStr);
		}

		if (SessionBitsData->GetType() == EOnlineKeyValuePairDataType::UInt32)
		{
			uint32 SessionBits;
			SessionBitsData->GetValue(SessionBits);

			int32 NumBits = 0;
			SessionSettings.bShouldAdvertise = !!(SessionBits & (1 << NumBits)); NumBits++;
			SessionSettings.bAllowJoinInProgress = !!(SessionBits & (1 << NumBits)); NumBits++;
			SessionSettings.bAllowInvites = !!(SessionBits & (1 << NumBits)); NumBits++;
			SessionSettings.bAllowJoinViaPresence = !!(SessionBits & (1 << NumBits)); NumBits++;
			SessionSettings.bAllowJoinViaPresenceFriendsOnly = !!(SessionBits & (1 << NumBits)); NumBits++;
		}

		if (RemoteBuildUniqueId->GetType() == EOnlineKeyValuePairDataType::Int32)
		{
			RemoteBuildUniqueId->GetValue(SessionSettings.BuildUniqueId);
		}

		static FName SessionIdFName(RAIL_SESSION_ID_KEY);
		static FName OwningUserIdFName(RAIL_SESSION_OWNING_USER_ID_KEY);
		static FName SessionBitsFName(RAIL_SESSION_SESSIONBITS_KEY);
		static FName BuildUniqueIdFName(RAIL_SESSION_BUILDUNIQUEID_KEY);

		for (TPair<FString, FVariantData> Pair : InResult.Metadata)
		{
			FName SettingKey(*Pair.Key);
			if (SettingKey != SessionIdFName &&
				SettingKey != OwningUserIdFName &&
				SettingKey != SessionBitsFName &&
				SettingKey != BuildUniqueIdFName)
			{
				FOnlineSessionSetting SessionSetting(Pair.Value, EOnlineDataAdvertisementType::ViaOnlineService);
				SessionSettings.Set(SettingKey, SessionSetting);
			}
		}
	}

#if !UE_BUILD_SHIPPING
	DumpSession(&NewSearchResult->Session);
#endif

	int32 BuildUniqueId = GetBuildUniqueId();
	if (NewSearchResult->Session.SessionSettings.BuildUniqueId == 0 ||
		NewSearchResult->Session.SessionSettings.BuildUniqueId != BuildUniqueId ||
		!SessionInfo->IsValid())
	{
		const FString SessionIdStr = SessionInfo->SessionId.IsValid() ? SessionInfo->SessionId->ToString() : TEXT("InvalidSession");

		if (!SessionInfo->SessionId.IsValid())
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Rejecting search result [%s]: invalid session id"), *SessionIdStr);
		}

		if (NewSearchResult->Session.SessionSettings.BuildUniqueId == 0 ||
			NewSearchResult->Session.SessionSettings.BuildUniqueId != BuildUniqueId)
		{
			UE_LOG_ONLINE_SESSION(Verbose, TEXT("Rejecting search result [%s]: invalid build id %d != %d"),
				*SessionIdStr,
				BuildUniqueId,
				NewSearchResult->Session.SessionSettings.BuildUniqueId);
		}

		// Remove the failed element
		InSearch->SearchResults.RemoveAtSwap(InSearch->SearchResults.Num() - 1);
	}
}

bool FOnlineSessionTencentRail::FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend)
{
	bool bSuccess = false;
	const FUniqueNetIdRail& RailFriendId = (const FUniqueNetIdRail&)Friend;

	if (RailFriendId.IsValid())
	{
		TWeakPtr<FOnlineSessionTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
		FOnOnlineAsyncTaskRailGetUserInviteComplete CompletionDelegate = FOnOnlineAsyncTaskRailGetUserInviteComplete::CreateLambda([LocalUserNum, LocalWeakThis](const FGetUserInviteTaskResult& Result)
		{
			FOnlineSessionTencentRailPtr StrongThis = StaticCastSharedPtr<FOnlineSessionTencentRail>(LocalWeakThis.Pin());
			if (StrongThis.IsValid())
			{
				if (Result.Error.WasSuccessful() && Result.Metadata.Num() && !Result.Commandline.IsEmpty())
				{
					TSharedRef<FOnlineSessionSearch> SessionSearch = MakeShared<FOnlineSessionSearch>();
					StrongThis->ParseSearchResult(SessionSearch, Result);
					StrongThis->TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, Result.Error.WasSuccessful() && (SessionSearch->SearchResults.Num() > 0), SessionSearch->SearchResults);
				}
				else
				{
					TArray<FOnlineSessionSearchResult> EmptyResult;
					StrongThis->TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, false, EmptyResult);
				}
			}
		});

		FOnlineAsyncTaskRailGetUserInvite* NewTask = new FOnlineAsyncTaskRailGetUserInvite(TencentSubsystem, RailFriendId, CompletionDelegate);
		UE_LOG_ONLINE_SESSION(Verbose, TEXT("%s"), *NewTask->ToString());
		TencentSubsystem->QueueAsyncTask(NewTask);
		bSuccess = true;
	}

	if (!bSuccess)
	{
		TWeakPtr<FOnlineSessionTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
		TencentSubsystem->ExecuteNextTick([LocalWeakThis, LocalUserNum, bSuccess]()
		{
			FOnlineSessionTencentRailPtr StrongThis = StaticCastSharedPtr<FOnlineSessionTencentRail>(LocalWeakThis.Pin());
			if (StrongThis.IsValid())
			{
				TArray<FOnlineSessionSearchResult> EmptyResult;
				StrongThis->TriggerOnFindFriendSessionCompleteDelegates(LocalUserNum, bSuccess, EmptyResult);
			}
		});
	}

	return bSuccess;
}

bool FOnlineSessionTencentRail::FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend)
{
	return FindFriendSession(GetLocalUserIdx(LocalUserId), Friend);
}

bool FOnlineSessionTencentRail::FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList)
{
	/** NYI */
	return false;
}

bool FOnlineSessionTencentRail::SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend)
{
	TArray< FUniqueNetIdRef > Friends;
	Friends.Add(Friend.AsShared());
	return SendSessionInviteToFriends(LocalUserNum, SessionName, Friends);
}

bool FOnlineSessionTencentRail::SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend)
{
	TArray< FUniqueNetIdRef > Friends;
	Friends.Add(Friend.AsShared());
	return SendSessionInviteToFriends(LocalUserId, SessionName, Friends);
}

bool FOnlineSessionTencentRail::SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	IOnlineIdentityPtr IdentityInt = TencentSubsystem->GetIdentityInterface();
	if (IdentityInt.IsValid())
	{
		FUniqueNetIdPtr UserId = IdentityInt->GetUniquePlayerId(LocalUserNum);
		if (UserId.IsValid())
		{
			return SendSessionInviteToFriends(*UserId, SessionName, Friends);
		}
	}

	return false;
}

bool FOnlineSessionTencentRail::SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends)
{
	if (!SessionName.IsNone() && Friends.Num() > 0)
	{
		FUniqueNetIdRailRef UserIdRail = StaticCastSharedRef<const FUniqueNetIdRail>(LocalUserId.AsShared());

		TWeakPtr<FOnlineSubsystemTencent, ESPMode::ThreadSafe> LocalWeakSubsystem(TencentSubsystem->AsShared());
		// This function doesn't work for querying your own details
		//FOnlineAsyncTaskRailGetInviteDetails* OuterTask = new FOnlineAsyncTaskRailGetInviteDetails(TencentSubsystem, *UserIdRail, FOnOnlineAsyncTaskRailGetInviteDetailsComplete::CreateLambda([LocalWeakSubsystem, UserIdRail, Friends](const FGetInviteDetailsTaskResult& OuterResult)
		FOnlineAsyncTaskRailGetInviteCommandline* OuterTask = new FOnlineAsyncTaskRailGetInviteCommandline(TencentSubsystem, *UserIdRail, FOnOnlineAsyncTaskRailGetInviteCommandLineComplete::CreateLambda([LocalWeakSubsystem, UserIdRail, Friends](const FGetInviteCommandLineTaskResult& OuterResult)
		{
			FOnlineSubsystemTencentPtr StrongSubsystem = StaticCastSharedPtr<FOnlineSubsystemTencent>(LocalWeakSubsystem.Pin());
			if (StrongSubsystem.IsValid())
			{
				if (OuterResult.Error.WasSuccessful())
				{
					FOnlineAsyncTaskRailSendInvite* InnerTask = new FOnlineAsyncTaskRailSendInvite(StrongSubsystem.Get(), OuterResult.Commandline, Friends, FOnOnlineAsyncTaskRailSendInviteComplete::CreateLambda([LocalWeakSubsystem](const FSendInviteTaskResult& InnerResult)
					{
						if (InnerResult.Error.WasSuccessful())
						{
							UE_LOG_ONLINE_SESSION(Display, TEXT("Invite sent"));
						}
						else
						{
							UE_LOG_ONLINE_SESSION(Display, TEXT("Failed to send invite"));
						}
					}));
					StrongSubsystem->QueueAsyncTask(InnerTask);
				}
				else
				{
					UE_LOG_ONLINE_SESSION(Display, TEXT("Failed to get invite details to send invite"));
				}
			}
		}));

		TencentSubsystem->QueueAsyncTask(OuterTask);
		return true;
	}
	else
	{
		UE_LOG_ONLINE_SESSION(Display, TEXT("Invalid params sending invite Session %s FriendCount: %d"), *SessionName.ToString(), Friends.Num());
	}

	return false;
}


#endif // WITH_TENCENT_RAIL_SDK
