// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineStoreTencent.h"

#if WITH_TENCENT_RAIL_SDK

#include "OnlineSubsystemTencent.h"
#include "OnlineAsyncTasksTencent.h"

FOnlineStoreTencent::FOnlineStoreTencent(FOnlineSubsystemTencent* InSubsystem)
	: Subsystem(InSubsystem)
{
	check(Subsystem != nullptr);
}

void FOnlineStoreTencent::QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate)
{
	// NYI
	Subsystem->ExecuteNextTick([Delegate]()
	{
		Delegate.ExecuteIfBound(false, TEXT("QueryCategories Not Implemented"));
	});
}

void FOnlineStoreTencent::QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	FOnOnlineAsyncTaskRailRequestAllPurchasableProductsComplete CompletionDelegate;
	CompletionDelegate.BindThreadSafeSP(this, &FOnlineStoreTencent::QueryOffers_Complete, OfferIds, Delegate);

	FOnlineAsyncTaskRailRequestAllPurchasableProducts* AsyncTask = new FOnlineAsyncTaskRailRequestAllPurchasableProducts(Subsystem, CompletionDelegate);
	Subsystem->QueueAsyncTask(AsyncTask);
}

void FOnlineStoreTencent::QueryOffers_Complete(const FRequestAllPurchasableProductsTaskResult& Result, const TArray<FUniqueOfferId> RequestedOfferIds, FOnQueryOnlineStoreOffersComplete Delegate)
{
	if (Result.Error.WasSuccessful())
	{
		for (const FOnlineStoreOfferRef& Offer : Result.Offers)
		{
			CachedOffers.Emplace(Offer->OfferId, Offer);
		}
	}
	else
	{
		CachedOffers.Empty();
	}
	Delegate.ExecuteIfBound(Result.Error.WasSuccessful(), RequestedOfferIds, Result.Error.ErrorCode);
}

void FOnlineStoreTencent::QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate)
{
	// NYI
	Subsystem->ExecuteNextTick([Delegate]()
	{
		Delegate.ExecuteIfBound(false, TArray<FUniqueOfferId>(), TEXT("QueryOffersByFilter Not Implemented"));
	});
}

void FOnlineStoreTencent::GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const
{
	// NYI
	OutCategories.Empty();
}

void FOnlineStoreTencent::GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const
{
	CachedOffers.GenerateValueArray(OutOffers);
}

TSharedPtr<FOnlineStoreOffer> FOnlineStoreTencent::GetOffer(const FUniqueOfferId& OfferId) const
{
	TSharedPtr<FOnlineStoreOffer> Result;
	const FOnlineStoreOfferRef* const Offer = CachedOffers.Find(OfferId);
	if (Offer)
	{
		Result = *Offer;
	}
	return Result;
}

#endif // WITH_TENCENT_RAIL_SDK
