// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TENCENTSDK

#include "Interfaces/OnlineStoreInterfaceV2.h"
#include "OnlineSubsystemTencentPackage.h"

class FOnlineSubsystemTencent;
struct FRequestAllPurchasableProductsTaskResult;

/**
 * Implementation for online store via WeGame
 */
class FOnlineStoreTencent :
	public IOnlineStoreV2,
	public TSharedFromThis<FOnlineStoreTencent, ESPMode::ThreadSafe>
{
public:
	/** Destructor */
	virtual ~FOnlineStoreTencent() = default;

	// Begin IOnlineStoreV2
	virtual void QueryCategories(const FUniqueNetId& UserId, const FOnQueryOnlineStoreCategoriesComplete& Delegate) override;
	virtual void GetCategories(TArray<FOnlineStoreCategory>& OutCategories) const override;
	virtual void QueryOffersByFilter(const FUniqueNetId& UserId, const FOnlineStoreFilter& Filter, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void QueryOffersById(const FUniqueNetId& UserId, const TArray<FUniqueOfferId>& OfferIds, const FOnQueryOnlineStoreOffersComplete& Delegate) override;
	virtual void GetOffers(TArray<FOnlineStoreOfferRef>& OutOffers) const override;
	virtual TSharedPtr<FOnlineStoreOffer> GetOffer(const FUniqueOfferId& OfferId) const override;
	// End IOnlineStoreV2

PACKAGE_SCOPE:
	/**
	 * Constructor
	 * @param InSubsystem the owning subsystem
	 */
	FOnlineStoreTencent(FOnlineSubsystemTencent* InSubsystem);

private:
	/** Default constructor disabled */
	FOnlineStoreTencent() = delete;
	
	/** Owning subsystem */
	FOnlineSubsystemTencent* const Subsystem;

	void QueryOffers_Complete(const FRequestAllPurchasableProductsTaskResult& Result, const TArray<FUniqueOfferId> RequestedOfferIds, FOnQueryOnlineStoreOffersComplete Delegate);

	/** Map of offers queried via QueryOffersById */
	TMap<FUniqueOfferId, FOnlineStoreOfferRef> CachedOffers;
};

typedef TSharedPtr<FOnlineStoreTencent, ESPMode::ThreadSafe> FOnlineStoreTencentPtr;

#endif // WITH_TENCENTSDK
