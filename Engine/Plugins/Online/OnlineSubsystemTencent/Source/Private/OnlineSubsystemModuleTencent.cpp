// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemTencentModule.h"
#include "OnlineSubsystemTencent.h"

IMPLEMENT_MODULE(FOnlineSubsystemTencentModule, OnlineSubsystemTencent);

#if WITH_TENCENTSDK
/**
 * Class responsible for creating instance(s) of the subsystem
 */
class FOnlineFactoryTencent : public IOnlineFactory
{
public:

	FOnlineFactoryTencent() : bOnlineSubCreateFailed(false) {}
	virtual ~FOnlineFactoryTencent() {}

	virtual IOnlineSubsystemPtr CreateSubsystem(FName InstanceName)
	{
		FOnlineSubsystemTencentPtr OnlineSub;
		if (!bOnlineSubCreateFailed)
		{
			OnlineSub = MakeShared<FOnlineSubsystemTencent, ESPMode::ThreadSafe>(InstanceName);
			if (OnlineSub->IsEnabled())
			{
				if (!OnlineSub->Init())
				{
					UE_LOG_ONLINE(Warning, TEXT("Tencent API failed to initialize instance %s!"), *InstanceName.ToString());
					OnlineSub->Shutdown();
					OnlineSub.Reset();
				}
			}
			else
			{
				static bool bHasAlerted = false;
				if (!bHasAlerted)
				{
					// Alert once with a warning for visibility (should be at the beginning)
					UE_LOG_ONLINE(Log, TEXT("Tencent API disabled."));
					bHasAlerted = true;
				}
				OnlineSub->Shutdown();
				OnlineSub.Reset();
			}
		}

		if (!OnlineSub.IsValid())
		{
			bOnlineSubCreateFailed = true;
		}
		return OnlineSub;
	}

	/** Have we ever failed to create an instance? */
	bool bOnlineSubCreateFailed;
};

#endif

void FOnlineSubsystemTencentModule::StartupModule()
{
#if WITH_TENCENTSDK
	UE_LOG_ONLINE(Log, TEXT("Tencent Startup!"));

	TencentFactory = new FOnlineFactoryTencent();

	// Create and register our singleton factory with the main online subsystem for easy access
	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.RegisterPlatformService(TENCENT_SUBSYSTEM, TencentFactory);
#endif
}

void FOnlineSubsystemTencentModule::ShutdownModule()
{
#if WITH_TENCENTSDK
	UE_LOG_ONLINE(Log, TEXT("Tencent Shutdown!"));

	FOnlineSubsystemModule& OSS = FModuleManager::GetModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	OSS.UnregisterPlatformService(TENCENT_SUBSYSTEM);

	delete TencentFactory;
	TencentFactory = NULL;
#endif // WITH_TENCENTSDK
}

