// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemTencent.h"

#if WITH_TENCENTSDK
#include "Interfaces/IMessageSanitizerInterface.h"
#include "OnlineAsyncTaskManagerTencent.h"
#include "Interfaces/OnlineEntitlementsInterface.h"
#include "OnlineIdentityTencent.h"
#include "HAL/RunnableThread.h"
#include "Interfaces/OnlineExternalUIInterface.h"
#include "OnlineIdentityTencent.h"
#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystemTencent.h"
#include "Interfaces/OnlineTitleFileInterface.h"
#include "Interfaces/OnlineUserCloudInterface.h"
#include "Interfaces/VoiceInterface.h"
#include "OnlineSessionTencent.h"
#include "Trace/Trace.inl"

#if WITH_TENCENT_RAIL_SDK
#include "Features/IModularFeatures.h"
#include "Interfaces/OnlinePlayTimeLimit.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "OnlineExternalUITencent.h"
#include "OnlineFriendsTencent.h"
#include "OnlineMessageSanitizerTencent.h"
#include "OnlinePlayTimeLimitTencent.h"
#include "OnlinePresenceTencent.h"
#include "OnlinePurchaseTencent.h"
#include "OnlineSessionTencentRail.h"
#include "OnlineStoreTencent.h"
#include "OnlineSubsystemTencentPrivate.h"
#include "OnlineUserTencent.h"
#include "PlayTimeLimitImpl.h"
#endif

IOnlineSessionPtr FOnlineSubsystemTencent::GetSessionInterface() const
{
	return TencentSession;
}

IOnlineFriendsPtr FOnlineSubsystemTencent::GetFriendsInterface() const
{
#if WITH_TENCENT_RAIL_SDK
	return TencentFriends;
#else
	return nullptr;
#endif
}

IOnlinePartyPtr FOnlineSubsystemTencent::GetPartyInterface() const
{
	return nullptr;
}

IOnlineGroupsPtr FOnlineSubsystemTencent::GetGroupsInterface() const
{
	return nullptr;
}

IOnlineSharedCloudPtr FOnlineSubsystemTencent::GetSharedCloudInterface() const
{
	return nullptr;
}

IOnlineUserCloudPtr FOnlineSubsystemTencent::GetUserCloudInterface() const
{
	return nullptr;
}

IOnlineEntitlementsPtr FOnlineSubsystemTencent::GetEntitlementsInterface() const
{
	return nullptr;
}

IOnlineLeaderboardsPtr FOnlineSubsystemTencent::GetLeaderboardsInterface() const
{
	return nullptr;
}

IOnlineVoicePtr FOnlineSubsystemTencent::GetVoiceInterface() const
{
	return nullptr;
}

IOnlineExternalUIPtr FOnlineSubsystemTencent::GetExternalUIInterface() const
{
#if WITH_TENCENT_RAIL_SDK
	return TencentExternalUI;
#else
	return nullptr;
#endif
}

IOnlineTimePtr FOnlineSubsystemTencent::GetTimeInterface() const
{
	return nullptr;
}

IOnlineIdentityPtr FOnlineSubsystemTencent::GetIdentityInterface() const
{
	return TencentIdentity;
}

IOnlineTitleFilePtr FOnlineSubsystemTencent:: GetTitleFileInterface() const
{
	return nullptr;
}

IOnlineStoreV2Ptr FOnlineSubsystemTencent::GetStoreV2Interface() const
{
#if	WITH_TENCENT_RAIL_SDK
	return TencentStore;
#else
	return nullptr;
#endif
}

IOnlinePurchasePtr FOnlineSubsystemTencent::GetPurchaseInterface() const
{
#if	WITH_TENCENT_RAIL_SDK
	return TencentPurchase;
#else
	return nullptr;
#endif
}

IOnlineEventsPtr FOnlineSubsystemTencent::GetEventsInterface() const
{
	return nullptr;
}

IOnlineAchievementsPtr FOnlineSubsystemTencent::GetAchievementsInterface() const
{
	return nullptr;
}

IOnlineSharingPtr FOnlineSubsystemTencent::GetSharingInterface() const
{
	return nullptr;
}

IOnlineUserPtr FOnlineSubsystemTencent::GetUserInterface() const
{
#if	WITH_TENCENT_RAIL_SDK
	return TencentUser;
#else
	return nullptr;
#endif
}

IOnlineMessagePtr FOnlineSubsystemTencent::GetMessageInterface() const
{
	return nullptr;
}

IOnlinePresencePtr FOnlineSubsystemTencent::GetPresenceInterface() const
{
#if	WITH_TENCENT_RAIL_SDK
	return TencentPresence;
#else
	return nullptr;
#endif
}

IOnlineChatPtr FOnlineSubsystemTencent::GetChatInterface() const
{
	return nullptr;
}

IOnlineStatsPtr FOnlineSubsystemTencent::GetStatsInterface() const
{
	return nullptr;
}

IOnlineTurnBasedPtr FOnlineSubsystemTencent::GetTurnBasedInterface() const
{
	return nullptr;
}

IOnlineTournamentPtr FOnlineSubsystemTencent::GetTournamentInterface() const
{
	return nullptr;
}

void FOnlineSubsystemTencent::QueueAsyncTask(FOnlineAsyncTask* AsyncTask)
{
	check(OnlineAsyncTaskThreadRunnable);
	OnlineAsyncTaskThreadRunnable->AddToInQueue(AsyncTask);
}

void FOnlineSubsystemTencent::QueueAsyncOutgoingItem(FOnlineAsyncItem* AsyncItem)
{
	check(OnlineAsyncTaskThreadRunnable);
	OnlineAsyncTaskThreadRunnable->AddToOutQueue(AsyncItem);
}

void FOnlineSubsystemTencent::QueueAsyncParallelTask(FOnlineAsyncTask* AsyncTask)
{
	check(OnlineAsyncTaskThreadRunnable);
	OnlineAsyncTaskThreadRunnable->AddToParallelTasks(AsyncTask);
}

#if WITH_TENCENT_RAIL_SDK
void FOnlineSubsystemTencent::AddToMetadataCache(const TMap<FString, FString>& InMetadata)
{
	MetadataDataCache.Append(InMetadata);
}
#endif

FText FOnlineSubsystemTencent::GetOnlineServiceName() const
{
	return NSLOCTEXT("FOnlineSubsystemTencent", "OnlineServiceName", "Tencent");
}

FText FOnlineSubsystemTencent::GetSocialPlatformName() const
{
	return NSLOCTEXT("FOnlineSubsystemTencent", "SocialPlatformName", "WeGame Friends");
}

bool FOnlineSubsystemTencent::Init()
{
	UE_LOG_ONLINE(Verbose, TEXT("FOnlineSubsystemTencent::Init() Name: %s"), *InstanceName.ToString());


	// Initialize Rail SDK
	if (!InitRailSdk())
	{
		UE_LOG_ONLINE(Warning, TEXT("Failed to initialize Rail"));
		return false;
	}

#if WITH_TENCENT_RAIL_SDK
	if (IModularFeatures::Get().IsModularFeatureAvailable(IOnlinePlayTimeLimit::GetModularFeatureName()))
	{
		// Register delegate to create users for the Anti-Addiction system.
		FPlayTimeLimitImpl::Get().OnRequestCreateUser.BindLambda([](const FUniqueNetId& UserId)->FPlayTimeLimitUser*
		{
			return new FOnlinePlayTimeLimitUserTencentRail(UserId.AsShared());
		});
	}
#endif

	bool bInitSuccess = true;

	// Create the online async task thread (after RAIL to register events)
	OnlineAsyncTaskThreadRunnable = new FOnlineAsyncTaskManagerTencent(this);
	check(OnlineAsyncTaskThreadRunnable);
	OnlineAsyncTaskThread = FRunnableThread::Create(OnlineAsyncTaskThreadRunnable, *FString::Printf(TEXT("OnlineAsyncTaskThreadTencent %s"), *InstanceName.ToString()), 128 * 1024, TPri_Normal);
	check(OnlineAsyncTaskThread);
	UE_LOG_ONLINE(Verbose, TEXT("Created thread (ID:%d)."), OnlineAsyncTaskThread->GetThreadID());

	TencentIdentity = MakeShared<FOnlineIdentityTencent, ESPMode::ThreadSafe>(this);

#if WITH_TENCENT_RAIL_SDK
	TencentSession = MakeShared<FOnlineSessionTencentRail, ESPMode::ThreadSafe>(this);

	if (!TencentSession->Init())
	{
		UE_LOG_ONLINE(Warning, TEXT("Failed to initialize session interface"));
		TencentSession.Reset();
		bInitSuccess = false;
	}
	
	TencentFriends = MakeShared<FOnlineFriendsTencent, ESPMode::ThreadSafe>(this);
	if (!TencentFriends->Init())
	{
		UE_LOG_ONLINE(Warning, TEXT("Failed to initialize friends interface"));
		TencentFriends.Reset();
		bInitSuccess = false;
	}
	TencentPresence = MakeShared<FOnlinePresenceTencent, ESPMode::ThreadSafe>(this);
	if (!TencentPresence->Init())
	{
		UE_LOG_ONLINE(Warning, TEXT("Failed to initialize presence interface"));
		TencentPresence.Reset();
		bInitSuccess = false;
	}
	TencentExternalUI = MakeShared<FOnlineExternalUITencent, ESPMode::ThreadSafe>(this);
	TencentUser = MakeShared<FOnlineUserTencent, ESPMode::ThreadSafe>(this);
	TencentMessageSanitizer = MakeShared<FMessageSanitizerTencent, ESPMode::ThreadSafe>(this);
	TencentPurchase = MakeShared<FOnlinePurchaseTencent, ESPMode::ThreadSafe>(this);
	TencentStore = MakeShared<FOnlineStoreTencent, ESPMode::ThreadSafe>(this);
#endif

	// update services based on user login/logout events
	OnLoginChangedHandle = TencentIdentity->AddOnLoginChangedDelegate_Handle(FOnLoginChangedDelegate::CreateThreadSafeSP(this, &FOnlineSubsystemTencent::OnLoginChanged));

	return bInitSuccess;
}

/**
 * @param 'uint32_t' is the security level, defined in RailWarningMessageLevel
 * @param 'const char*' is the message
 */ 
void RailWarningMessageCallback(uint32_t Level, const char* Msg)
{
	UE_LOG_ONLINE(Warning, TEXT("RailWarning: [%d] %s"), Level, Msg ? UTF8_TO_TCHAR(Msg) : TEXT("NoWarning"));
}

bool FOnlineSubsystemTencent::UsesRailSdk() const
{
	const bool bUsesRailSdk = WITH_TENCENT_RAIL_SDK && !IsDedicated();
	return bUsesRailSdk;
}

bool FOnlineSubsystemTencent::InitRailSdk()
{
	bool bResult = false;
	if (UsesRailSdk())
	{
#if WITH_TENCENT_RAIL_SDK
		// Make sure the Rail SDK (Wegame) dll is loaded/initialized	
		RailSdkWrapper& Wrapper = RailSdkWrapper::Get();
		if (Wrapper.Load())
		{
			// Game id config
			FString RailGameIdStr;
			if (GConfig->GetString(TEXT("OnlineSubsystemTencent"), TEXT("RailGameId"), RailGameIdStr, GEngineIni) &&
				!RailGameIdStr.IsEmpty())
			{
				LexFromString(RailGameId, *RailGameIdStr);
				const bool bRailNeedsRestart = Wrapper.RailNeedRestartAppForCheckingEnvironment(rail::RailGameID(RailGameId));
				if (!bRailNeedsRestart)
				{
					if (Wrapper.RailInitialize())
					{
						rail::IRailFactory* const RailFactory = Wrapper.RailFactory();
						if (RailFactory)
						{
							rail::IRailUtils* const RailUtils = RailFactory->RailUtils();
							if (RailUtils)
							{
								RailUtils->SetWarningMessageCallback(RailWarningMessageCallback);
							}
							// Log if we are not Online
							rail::IRailSystemHelper* const RailSystemHelper = RailFactory->RailSystemHelper();
							if (RailSystemHelper)
							{
								const rail::RailSystemState SystemState = RailSystemHelper->GetPlatformSystemState();
								switch (SystemState)
								{
								case rail::RailSystemState::kSystemStatePlatformOnline:
									break;
								default:
									UE_LOG_ONLINE(Warning, TEXT("InitRailSdk: Rail Platform System State is %s, this will impact your ability to login"), *LexToString(SystemState));
								}
							}

							// Clear my metadata on startup in case of a previous crash while we load
							rail::IRailFriends* Friends = RailFactory->RailFriends();
							if (Friends)
							{
								// Presence and session interfaces need to clear this data on shutdown
								rail::RailResult RailResult = Friends->AsyncClearAllMyMetadata(rail::RailString());
							}
						}
						bResult = true;
					}
					else
					{
						UE_LOG_ONLINE(Warning, TEXT("RailInitialize failed with RailGameid=%llu"), RailGameId);
					}
				}
				else
				{
					UE_LOG_ONLINE(Warning, TEXT("RailNeedRestartAppForCheckingEnvironment failed with RailGameid=%llu"), RailGameId);
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
					if (FCString::Strstr(FCommandLine::Get(), TEXT("rail_debug_mode")))
					{
						UE_LOG_ONLINE(Warning, TEXT("When running with --rail_debug_mode, be sure you are running the game as an administrator"));
					}
					else
					{
						UE_LOG_ONLINE(Warning, TEXT("Was the game launched through WeGame?"))
					}
#endif
				}
			}
			else
			{
				UE_LOG_ONLINE(Warning, TEXT("Invalid RailGameId. Please set [OnlineSubsystemTencent] RailGameId=<Id> in your Engine.ini"));
			}
		}
		else
		{
			UE_LOG_ONLINE(Warning, TEXT("Failed to load the Rail SDK Wrapper"));
		}
#endif
	}
	else
	{
		// no errors if we don't use RailSDK
		bResult = true;
	}
	return bResult;
}

void FOnlineSubsystemTencent::ShutdownRailSdk()
{
	if (UsesRailSdk())
	{
#if WITH_TENCENT_RAIL_SDK
		RailSdkWrapper& Wrapper = RailSdkWrapper::Get();
		if (Wrapper.IsInitialized())
		{
			rail::IRailFactory* RailFactory = Wrapper.RailFactory();
			if (RailFactory)
			{
				rail::IRailFriends* Friends = RailFactory->RailFriends();
				if (Friends)
				{
					// Presence and session interfaces need to clear this data on shutdown
					rail::RailResult RailResult = Friends->AsyncClearAllMyMetadata(rail::RailString());
				}
			}
		}

		Wrapper.RailFinalize();
		Wrapper.Shutdown();
#endif
	}
}

void FOnlineSubsystemTencent::OnLoginChanged(int32 LocalUserNum)
{
}

void FOnlineSubsystemTencent::PreUnload()
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemTencent::Preunload() Name: %s"), *InstanceName.ToString());
}

bool FOnlineSubsystemTencent::Shutdown()
{
	UE_LOG_ONLINE(VeryVerbose, TEXT("FOnlineSubsystemTencent::Shutdown() Name: %s"), *InstanceName.ToString());

	FOnlineSubsystemImpl::Shutdown();

#if WITH_TENCENT_RAIL_SDK
	if (TencentPresence.IsValid())
	{
		TencentPresence->Shutdown();
	}
#endif

	if (TencentSession.IsValid())
	{
		TencentSession->Shutdown();
	}

	if (TencentIdentity.IsValid())
	{
		TencentIdentity->ClearOnLoginChangedDelegate_Handle(OnLoginChangedHandle);
	}

#define DESTRUCT_INTERFACE(Interface) \
	if (Interface.IsValid()) \
	{ \
		ensure(Interface.IsUnique()); \
		Interface.Reset(); \
	}

	// Destruct the interfaces
#if WITH_TENCENT_RAIL_SDK
	DESTRUCT_INTERFACE(TencentStore);
	DESTRUCT_INTERFACE(TencentPurchase);
	DESTRUCT_INTERFACE(TencentUser);
	DESTRUCT_INTERFACE(TencentExternalUI);
	DESTRUCT_INTERFACE(TencentPresence);
	DESTRUCT_INTERFACE(TencentFriends);
	DESTRUCT_INTERFACE(TencentMessageSanitizer);
#endif
	DESTRUCT_INTERFACE(TencentSession);
	DESTRUCT_INTERFACE(TencentIdentity);

	if (OnlineAsyncTaskThread)
	{
		// Destroy the online async task thread
		delete OnlineAsyncTaskThread;
		OnlineAsyncTaskThread = nullptr;
	}

	if (OnlineAsyncTaskThreadRunnable)
	{
		delete OnlineAsyncTaskThreadRunnable;
		OnlineAsyncTaskThreadRunnable = nullptr;
	}

#if WITH_TENCENT_RAIL_SDK
	// Shutdown / unload the Rail SDK (Wegame)
	ShutdownRailSdk();
#endif

	return true;
}

bool FOnlineSubsystemTencent::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FOnlineSubsystemTencent_Tick);
	
	if (!FOnlineSubsystemImpl::Tick(DeltaTime))
	{
		return false;
	}

	if (OnlineAsyncTaskThreadRunnable)
	{
		OnlineAsyncTaskThreadRunnable->GameTick();
	}

	if (TencentSession.IsValid())
	{
		TencentSession->Tick(DeltaTime);
	}

	return true;
}

FString FOnlineSubsystemTencent::GetAppId() const
{
	FString AppId;
#if WITH_TENCENT_RAIL_SDK
	AppId = LexToString(RailGameId);
#endif
	return AppId;
}

bool FOnlineSubsystemTencent::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FOnlineSubsystemImpl::Exec(InWorld, Cmd, Ar))
	{
		return true;
	}

	bool bWasHandled = false;
	if (FParse::Command(&Cmd, TEXT("TEST")))
	{
#if WITH_DEV_AUTOMATION_TESTS
		if (FParse::Command(&Cmd, TEXT("AUTH")))
		{
			bWasHandled = HandleAuthExecCommands(InWorld, Cmd, Ar);
		}
#endif // WITH_DEV_AUTOMATION_TESTS
	}
	else if (FParse::Command(&Cmd, TEXT("SESSION")))
	{
		bWasHandled = HandleSessionExecCommands(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("PRESENCE")))
	{
		bWasHandled = HandlePresenceExecCommands(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("USERS")))
	{
		bWasHandled = HandleUsersExecCommands(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("FRIENDS")))
	{
		bWasHandled = HandleFriendExecCommands(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("RAILSDKWRAPPER")))
	{
		bWasHandled = HandleRailSdkWrapperExecCommands(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPKEYS")))
	{
#if WITH_TENCENT_RAIL_SDK
		bool bLocalUser = true;
		FUniqueNetIdPtr UserId = nullptr;
		
		FString UserIdStr = FParse::Token(Cmd, true);
		if (!UserIdStr.IsEmpty())
		{
			UserId = FUniqueNetIdRail::Create(UserIdStr);
			bLocalUser = false;
		}
		else
		{
			UserId = GetFirstSignedInUser(TencentIdentity);
		}
		
		if (UserId.IsValid())
		{
			TArray<FString> OutDebugKeys;
			MetadataDataCache.GenerateKeyArray(OutDebugKeys);
			FUniqueNetIdRailPtr UserIdRail = StaticCastSharedPtr<const FUniqueNetIdRail>(UserId);
			FOnlineAsyncTaskRailGetUserMetadata* InnerTask = new FOnlineAsyncTaskRailGetUserMetadata(this, *UserIdRail, OutDebugKeys, FOnOnlineAsyncTaskRailGetUserMetadataComplete::CreateLambda([this, bLocalUser, OutDebugKeys](const FGetUserMetadataTaskResult& Result)
			{
				if (Result.Error.WasSuccessful())
				{
					UE_LOG_ONLINE(Display, TEXT("Metadata [%s]"), *Result.UserId->ToDebugString());
					UE_LOG_ONLINE(Display, TEXT("- Keys"));
					for (const TPair<FString, FVariantData>& Pair : Result.Metadata)
					{
						UE_LOG_ONLINE(Display, TEXT(" - [%s] %s"), *Pair.Key, *Pair.Value.ToString());
					}

					if (bLocalUser)
					{
						for (const FString& DebugKey : OutDebugKeys)
						{
							int32 SuffixIdx = INDEX_NONE;
							if (DebugKey.FindLastChar(TEXT('_'), SuffixIdx))
							{
								FString Key = DebugKey.Left(SuffixIdx);
								const FVariantData* Value = Result.Metadata.Find(Key);
								if (Value)
								{
									// Compare to cached value
									FString* CachedValue = MetadataDataCache.Find(DebugKey);
									if (CachedValue && (*CachedValue != Value->ToString()))
									{
										UE_LOG_ONLINE(Display, TEXT("Key does not match: %s [%s]:[%s]"), *DebugKey, **CachedValue, *Value->ToString());
									}
									else if (!CachedValue)
									{
										UE_LOG_ONLINE(Display, TEXT("Cache does not contain key: %s"), *DebugKey);
									}
								}
								else
								{
									UE_LOG_ONLINE(Display, TEXT("Result does not contain key: %s"), *DebugKey);
								}
							}
						}
					}
				}
				else
				{
					UE_LOG_ONLINE(Display, TEXT("Failed to get metadata"));
				}
			}));
			QueueAsyncTask(InnerTask);
		}
#endif
		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("AAS")))
	{
		UE_LOG_ONLINE(Warning, TEXT("AAS is no longer in OnlineSubsystemTencent, replace everything from the beginning of the command up to AAS with 'PlayTimeLimit' for example 'PlayTimeLimit NOTIFYNOW'"));
	}
	
	return bWasHandled;
}

bool FOnlineSubsystemTencent::HandleAuthExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

#if WITH_DEV_AUTOMATION_TESTS
	if (FParse::Command(&Cmd, TEXT("INFO")))
	{
		FString AuthType = FParse::Token(Cmd, false);
		bWasHandled = true;
	}
#endif // WITH_DEV_AUTOMATION_TESTS

	return bWasHandled;
}

bool FOnlineSubsystemTencent::HandleSessionExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;

#if WITH_TENCENT_RAIL_SDK
	if (FParse::Command(&Cmd, TEXT("DUMPMETADATA")))
	{
		// Retrieves session metadata based on the invite command line keys (tests individual steps, same as FOnlineAsyncTaskRailGetUserInvite)
		FUniqueNetIdPtr UserId = GetFirstSignedInUser(TencentIdentity);
		if (UserId.IsValid())
		{
			FUniqueNetIdRailPtr UserIdRail = StaticCastSharedPtr<const FUniqueNetIdRail>(UserId);

			TWeakPtr<FOnlineSubsystemTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
			FOnlineAsyncTaskRailGetInviteCommandline* OuterTask = new FOnlineAsyncTaskRailGetInviteCommandline(this, *UserIdRail, FOnOnlineAsyncTaskRailGetInviteCommandLineComplete::CreateLambda([LocalWeakThis, UserIdRail](const FGetInviteCommandLineTaskResult& OuterResult)
			{
				FOnlineSubsystemTencentPtr StrongThis = StaticCastSharedPtr<FOnlineSubsystemTencent>(LocalWeakThis.Pin());
				if (StrongThis.IsValid())
				{
					TArray<FString> Keys;
					if (OuterResult.Error.WasSuccessful())
					{
						FString CommandLineCopy = OuterResult.Commandline;
						CommandLineCopy.ParseIntoArray(Keys, RAIL_METADATA_KEY_SEPARATOR);
						FOnlineAsyncTaskRailGetUserMetadata* InnerTask = new FOnlineAsyncTaskRailGetUserMetadata(StrongThis.Get(), *UserIdRail, Keys, FOnOnlineAsyncTaskRailGetUserMetadataComplete::CreateLambda([UserIdRail, CommandLineCopy](const FGetUserMetadataTaskResult& InnerResult)
						{
							if (InnerResult.Error.WasSuccessful())
							{
								UE_LOG_ONLINE(Display, TEXT("Metadata [%s]"), *UserIdRail->ToDebugString());
								UE_LOG_ONLINE(Display, TEXT("- Commandline: %s"), *CommandLineCopy);
								UE_LOG_ONLINE(Display, TEXT("- Keys"));
								for (const TPair<FString, FVariantData>& Pair : InnerResult.Metadata)
								{
									UE_LOG_ONLINE(Display, TEXT(" - [%s] %s"), *Pair.Key, *Pair.Value.ToString());
								}
							}
							else
							{
								UE_LOG_ONLINE(Display, TEXT("Failed to get metadata"));
							}
						}));
						StrongThis->QueueAsyncTask(InnerTask);
					}
					else
					{
						UE_LOG_ONLINE(Display, TEXT("Failed to get command line metadata"));
					}
				}
			}));

			QueueAsyncTask(OuterTask);
		}

		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("METAINVITE")))
	{
		// Retrieve a session invite based on the local users session data
		FOnlineSessionTencentRailPtr TencentSessionRail = StaticCastSharedPtr<FOnlineSessionTencentRail>(TencentSession);
		if (TencentSessionRail.IsValid())
		{
			FUniqueNetIdPtr UserId = GetFirstSignedInUser(TencentIdentity);
			if (UserId.IsValid())
			{
				FUniqueNetIdRailPtr UserIdRail = StaticCastSharedPtr<const FUniqueNetIdRail>(UserId);

				TWeakPtr<FOnlineSubsystemTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
				FOnOnlineAsyncTaskRailGetUserInviteComplete CompletionDelegate = FOnOnlineAsyncTaskRailGetUserInviteComplete::CreateLambda([LocalWeakThis, UserIdRail](const FGetUserInviteTaskResult& Result)
				{
					FOnlineSubsystemTencentPtr StrongThis = StaticCastSharedPtr<FOnlineSubsystemTencent>(LocalWeakThis.Pin());
					if (StrongThis.IsValid())
					{
						if (Result.Error.WasSuccessful())
						{
							UE_LOG_ONLINE(Display, TEXT("Metadata [%s]"), *UserIdRail->ToDebugString());
							UE_LOG_ONLINE(Display, TEXT("- Commandline: %s"), *Result.Commandline);
							UE_LOG_ONLINE(Display, TEXT("- Keys"));
							for (const TPair<FString, FVariantData>& Pair : Result.Metadata)
							{
								UE_LOG_ONLINE(Display, TEXT(" - [%s] %s"), *Pair.Key, *Pair.Value.ToString());
							}
						}
						else
						{
							UE_LOG_ONLINE(Display, TEXT("Failed to get invite"));
						}
					}
				});

				FOnlineAsyncTaskRailGetUserInvite* NewTask = new FOnlineAsyncTaskRailGetUserInvite(this, *UserIdRail, CompletionDelegate);
				QueueAsyncTask(NewTask);
			}
		}

		bWasHandled = true;
	}
#endif //WITH_TENCENT_RAIL_SDK

	return bWasHandled;
}

bool FOnlineSubsystemTencent::HandlePresenceExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;
#if WITH_TENCENT_RAIL_SDK
	if (FParse::Command(&Cmd, TEXT("REPORTPLAYERS")))
	{
		TArray<FReportPlayedWithUser> ReportedPlayers;

		FString UserIdStr = FParse::Token(Cmd, true);
		FString PresenceStr = FParse::Token(Cmd, true);

		if (!UserIdStr.IsEmpty())
		{
			FUniqueNetIdRef ReportUserId(FUniqueNetIdRail::Create(UserIdStr));
			if (ReportUserId->IsValid())
			{
				ReportedPlayers.Emplace(ReportUserId, PresenceStr);
			}
			else
			{
				UE_LOG_ONLINE_FRIEND(Warning, TEXT("Invalid rail user specified"));
			}
		}
		else
		{
			if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
			{
				rail::RailArray<rail::RailFriendInfo> Friends;
				rail::RailResult Result = RailFriends->GetFriendsList(&Friends);
				if (Result == rail::kSuccess)
				{
					for (uint32 RailIdx = 0; RailIdx < Friends.size(); ++RailIdx)
					{
						const rail::RailFriendInfo& RailFriendInfo(Friends[RailIdx]);
						if (RailFriendInfo.friend_rail_id != rail::kInvalidRailId)
						{
							FUniqueNetIdRef FriendId(FUniqueNetIdRail::Create(RailFriendInfo.friend_rail_id));
							ReportedPlayers.Emplace(FriendId, PresenceStr);
						}
						else
						{
							UE_LOG_ONLINE_FRIEND(Warning, TEXT("Invalid friend in friends list"));
						}
					}
				}
			}
		}
		FOnlineAsyncTaskRailReportPlayedWithUsers* NewTask = new FOnlineAsyncTaskRailReportPlayedWithUsers(this, ReportedPlayers, FOnOnlineAsyncTaskRailReportPlayedWithUsersComplete::CreateLambda([](const FReportPlayedWithUsersTaskResult& Result)
		{
			UE_LOG_ONLINE(Display, TEXT("Reporting players complete: %s"), *Result.Error.ToLogString());
		}));
		QueueAsyncTask(NewTask);
		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("DUMP")))
	{
		// Retrieve local user presence
		if (TencentPresence.IsValid())
		{
			TWeakPtr<FOnlineSubsystemTencent, ESPMode::ThreadSafe> LocalWeakThis(AsShared());
			IOnlinePresence::FOnPresenceTaskCompleteDelegate CompletionDelegate = IOnlinePresence::FOnPresenceTaskCompleteDelegate::CreateLambda([LocalWeakThis](const FUniqueNetId& UserId, const bool bWasSuccessful)
			{
				UE_LOG_ONLINE(Display, TEXT("Presence [%s]"), *UserId.ToDebugString());

				FOnlineSubsystemTencentPtr StrongThis = StaticCastSharedPtr<FOnlineSubsystemTencent>(LocalWeakThis.Pin());
				if (StrongThis.IsValid())
				{
					if (bWasSuccessful)
					{
						IOnlinePresencePtr PresenceInt = StrongThis->GetPresenceInterface();
						if (PresenceInt.IsValid())
						{
							TSharedPtr<FOnlineUserPresence> UserPresence;
							if (PresenceInt->GetCachedPresence(UserId, UserPresence) == EOnlineCachedResult::Success &&
								UserPresence.IsValid())
							{
								UE_LOG_ONLINE(Display, TEXT("- %s"), *UserPresence->ToDebugString());
							}
							else
							{
								UE_LOG_ONLINE(Display, TEXT("Failed to get cached presence"));
							}
						}
					}
					else
					{
						UE_LOG_ONLINE(Display, TEXT("Failed to query presence"));
					}
				}
			});

			// Query and dump friends presence
			if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
			{
				rail::RailArray<rail::RailFriendInfo> Friends;
				rail::RailResult Result = RailFriends->GetFriendsList(&Friends);
				if (Result == rail::kSuccess)
				{
					TArray<FUniqueNetIdRef> FriendIds;
					for (uint32 RailIdx = 0; RailIdx < Friends.size(); ++RailIdx)
					{
						if (Friends[RailIdx].friend_rail_id != rail::kInvalidRailId)
						{
							FUniqueNetIdRailRef FriendId = FUniqueNetIdRail::Create(Friends[RailIdx].friend_rail_id);
							TencentPresence->QueryPresence(*FriendId, CompletionDelegate);
						}
					}
				}
			}
			
			// Query own presence
			FUniqueNetIdPtr UserId = GetFirstSignedInUser(TencentIdentity);
			if (UserId.IsValid())
			{
				TencentPresence->QueryPresence(*UserId, CompletionDelegate);
			}
		}
		bWasHandled = true;
	}
#endif //WITH_TENCENT_RAIL_SDK
	return bWasHandled;
}

bool FOnlineSubsystemTencent::HandleUsersExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;
	
#if WITH_TENCENT_RAIL_SDK
	const FUniqueNetIdPtr LocalUserId = GetFirstSignedInUser(TencentIdentity);
	const int32 LocalUserNum = LocalUserId.IsValid() ? TencentIdentity->GetPlatformUserIdFromUniqueNetId(*LocalUserId) : PLATFORMUSERID_NONE;
	if (LocalUserNum != (int32)PLATFORMUSERID_NONE)
	{
		bWasHandled = true;
		if (FParse::Command(&Cmd, TEXT("DUMPALL")))
		{
			TArray<TSharedRef<FOnlineUser>> Users;
			if (TencentUser->GetAllUserInfo(LocalUserNum, Users))
			{
				UE_LOG_ONLINE(Display, TEXT("%d users:"), Users.Num());
				int32 Counter = 0;
				for (const TSharedRef<FOnlineUser>& User : Users)
				{
					UE_LOG_ONLINE(Display, TEXT("  [%d]"), Counter++);
					UE_LOG_ONLINE(Display, TEXT("    id: %s"), *User->GetUserId()->ToDebugString());
					UE_LOG_ONLINE(Display, TEXT("    display name: %s"), *User->GetDisplayName());
				}
			}
			else
			{
				UE_LOG_ONLINE(Display, TEXT("GetAllUserInfo failed"));
			}
		}
		else if (FParse::Command(&Cmd, TEXT("QUERYUSER")))
		{
			FString UserIdString = Cmd;
			if (!UserIdString.IsEmpty())
			{
				FUniqueNetIdPtr UserId = TencentIdentity->CreateUniquePlayerId(UserIdString);
				if (UserId.IsValid())
				{
					TArray<FUniqueNetIdRef> UserIds;
					UserIds.Add(UserId.ToSharedRef());
					
					static FDelegateHandle DelegateHandle1;
					FOnQueryUserInfoCompleteDelegate OnQueryUserInfoCompleteDelegate = FOnQueryUserInfoCompleteDelegate::CreateLambda([this, LocalUserNum, UserId](int32, bool bSucceeded, const TArray< FUniqueNetIdRef >&, const FString& ErrorString)
					{
						if (bSucceeded)
						{
							TSharedPtr<FOnlineUser> OnlineUser = TencentUser->GetUserInfo(LocalUserNum, *UserId);
							if (OnlineUser.IsValid())
							{
								UE_LOG_ONLINE(Display, TEXT("User %s found"), *UserId->ToDebugString());
								UE_LOG_ONLINE(Display, TEXT("  Id: %s"), *OnlineUser->GetUserId()->ToDebugString());
								UE_LOG_ONLINE(Display, TEXT("  Display name: %s"), *OnlineUser->GetDisplayName());
							}
							else
							{
								UE_LOG_ONLINE(Display, TEXT("User %s not found"), *UserId->ToDebugString());
							}
						}
						else
						{
							UE_LOG_ONLINE(Display, TEXT("Query failed with error %s"), *ErrorString);
						}
						TencentUser->ClearOnQueryUserInfoCompleteDelegate_Handle(LocalUserNum, DelegateHandle1);
					});

					DelegateHandle1 = TencentUser->AddOnQueryUserInfoCompleteDelegate_Handle(LocalUserNum, OnQueryUserInfoCompleteDelegate);
					
					TencentUser->QueryUserInfo(LocalUserNum, UserIds);
				}
				else
				{
					UE_LOG_ONLINE(Display, TEXT("Failed to create rail id from '%s'"), *UserIdString);
				}
			}
			else
			{
				UE_LOG_ONLINE(Display, TEXT("Missing UserId"));
			}
		}
		else if (FParse::Command(&Cmd, TEXT("QUERYALLFRIENDS")))
		{
			// Just a test to see what we get when we query all our friends
			rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends();
			if (RailFriends)
			{
				rail::RailArray<rail::RailFriendInfo> Friends;
				rail::RailResult Result = RailFriends->GetFriendsList(&Friends);
				if (Result == rail::kSuccess)
				{
					TArray<FUniqueNetIdRef> UserIds;
					for (uint32 Index = 0; Index < Friends.size(); ++Index)
					{
						UserIds.Emplace(FUniqueNetIdRail::Create(Friends[Index].friend_rail_id));
					}

					static FDelegateHandle DelegateHandle2;
					FOnQueryUserInfoCompleteDelegate OnQueryUserInfoCompleteDelegate = FOnQueryUserInfoCompleteDelegate::CreateLambda([this, LocalUserNum, UserIds](int32, bool bSucceeded, const TArray< FUniqueNetIdRef >&, const FString& ErrorString)
					{
						if (bSucceeded)
						{
							for (const FUniqueNetIdRef& UserId : UserIds)
							{
								TSharedPtr<FOnlineUser> OnlineUser = TencentUser->GetUserInfo(LocalUserNum, *UserId);
								if (OnlineUser.IsValid())
								{
									UE_LOG_ONLINE(Display, TEXT("User %s found"), *UserId->ToDebugString());
									UE_LOG_ONLINE(Display, TEXT("  Id: %s"), *OnlineUser->GetUserId()->ToDebugString());
									UE_LOG_ONLINE(Display, TEXT("  Display name: %s"), *OnlineUser->GetDisplayName());
								}
								else
								{
									UE_LOG_ONLINE(Display, TEXT("User %s not found"), *UserId->ToDebugString());
								}
							}
						}
						else
						{
							UE_LOG_ONLINE(Display, TEXT("Query failed with error %s"), *ErrorString);
						}
						TencentUser->ClearOnQueryUserInfoCompleteDelegate_Handle(LocalUserNum, DelegateHandle2);
					});

					DelegateHandle2 = TencentUser->AddOnQueryUserInfoCompleteDelegate_Handle(LocalUserNum, OnQueryUserInfoCompleteDelegate);
					TencentUser->QueryUserInfo(LocalUserNum, UserIds);
				}
				else
				{
					UE_LOG_ONLINE(Display, TEXT("GetFriendsList failed %s"), *LexToString(Result));
				}
			}
			else
			{
				UE_LOG_ONLINE(Display, TEXT("Failed to get RailFriends"));
			}
		}
		else
		{
			bWasHandled = false;
		}
	}
	else
	{
		UE_LOG_ONLINE(Display, TEXT("You must be logged in first"));
	}
#endif //WITH_TENCENT_RAIL_SDK
	return bWasHandled;
}

bool FOnlineSubsystemTencent::HandleFriendExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	bool bWasHandled = false;
#if WITH_TENCENT_RAIL_SDK
	if (FParse::Command(&Cmd, TEXT("DUMPFRIENDS")))
	{
		if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
		{
			rail::RailArray<rail::RailFriendInfo> Friends;
			rail::RailResult Result = RailFriends->GetFriendsList(&Friends);

			TArray<FUniqueNetIdRef> FriendIds;
			for (uint32 RailIdx = 0; RailIdx < Friends.size(); ++RailIdx)
			{
				if (Friends[RailIdx].friend_rail_id != rail::kInvalidRailId)
				{
					FriendIds.Add(FUniqueNetIdRail::Create(Friends[RailIdx].friend_rail_id));
				}
			}

			if (FriendIds.Num() > 0)
			{
				TencentUser->AddOnQueryUserInfoCompleteDelegate_Handle(0, FOnQueryUserInfoCompleteDelegate::CreateLambda([this](int32 LocalUserNum, bool bWasSuccessful, const TArray<FUniqueNetIdRef>& UserIds, const FString& ErrorStr)
				{
					if (bWasSuccessful)
					{
						TArray<TSharedRef<FOnlineUser>> OutUsers;
						this->TencentUser->GetAllUserInfo(0, OutUsers);
						for (const TSharedRef<FOnlineUser>& User : OutUsers)
						{
							UE_LOG_ONLINE(Display, TEXT("  Id: %s"), *User->GetUserId()->ToDebugString());
							UE_LOG_ONLINE(Display, TEXT("  Display name: %s"), *User->GetDisplayName());
						}
					}
					else
					{
						UE_LOG_ONLINE(Display, TEXT("Query failed with error %s"), *ErrorStr);
					}
				}));

				TencentUser->QueryUserInfo(0, FriendIds);
			}
		}
		bWasHandled = true;
	}
	else if (FParse::Command(&Cmd, TEXT("DUMPPLAYEDGAMES")))
	{
		if (rail::IRailFriends* RailFriends = RailSdkWrapper::Get().RailFriends())
		{
			rail::RailArray<rail::RailFriendInfo> Friends;
			rail::RailResult Result = RailFriends->GetFriendsList(&Friends);

			FUniqueNetIdRailPtr LocalUserId = StaticCastSharedPtr<const FUniqueNetIdRail>(GetFirstSignedInUser(TencentIdentity));
			if (LocalUserId.IsValid())
			{
				for (uint32 RailIdx = 0; RailIdx < Friends.size(); ++RailIdx)
				{
					if (Friends[RailIdx].friend_rail_id != rail::kInvalidRailId)
					{
						if (Friends[RailIdx].friend_rail_id != (rail::RailID)(*LocalUserId))
						{
							UE_LOG_ONLINE(Display, TEXT("Query game played with %llu"), Friends[RailIdx].friend_rail_id.get_id());
							FUniqueNetIdRailPtr FriendId = FUniqueNetIdRail::Create(Friends[RailIdx].friend_rail_id);
							FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo* NewTask = new FOnlineAsyncTaskRailQueryFriendPlayedGamesInfo(this, *FriendId, FOnOnlineAsyncTaskRailQueryFriendPlayedGamesComplete::CreateLambda([](const FQueryFriendPlayedGamesTaskResult& Result)
							{
								UE_LOG_ONLINE(Display, TEXT("Query game played complete: %s"), *Result.Error.ToLogString());
							}));
							QueueAsyncTask(NewTask);
						}
					}
				}
			}
		}

		bWasHandled = true;
	}
#endif
	return bWasHandled;
}

bool FOnlineSubsystemTencent::HandleRailSdkWrapperExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if WITH_TENCENT_RAIL_SDK
	if (FParse::Command(&Cmd, TEXT("LOAD")))
	{
		const bool bResult = RailSdkWrapper::Get().Load();
		UE_LOG_ONLINE(Display, TEXT("RailSdkWrapper Load: %d"), static_cast<int32>(bResult));
	}
	else if (FParse::Command(&Cmd, TEXT("SHUTDOWN")))
	{
		RailSdkWrapper::Get().Shutdown();
		UE_LOG_ONLINE(Display, TEXT("RailSdkWrapper Shutdown"));
	}
	else if (FParse::Command(&Cmd, TEXT("NEEDRESTART")))
	{
		const bool bResult = RailSdkWrapper::Get().RailNeedRestartAppForCheckingEnvironment(rail::RailGameID(RailGameId));
		static const TCHAR* NeedsRestartString = TEXT("true (needs restart)");
		static const TCHAR* DoesNotNeedRestartString = TEXT("false (does not need restart)");
		const TCHAR* LogString = bResult ? NeedsRestartString : DoesNotNeedRestartString;
		UE_LOG_ONLINE(Display, TEXT("RailNeedRestartAppForCheckingEnvironment: %s for game id %llu"), LogString, RailGameId);
	}
	else if (FParse::Command(&Cmd, TEXT("INITIALIZE")))
	{
		const bool bResult = RailSdkWrapper::Get().RailInitialize();
		UE_LOG_ONLINE(Display, TEXT("RailInitialize: %d"), static_cast<int32>(bResult));
	}
	else if (FParse::Command(&Cmd, TEXT("PLAYER")))
	{
		return HandleRailSdkWrapperPlayerExecCommands(InWorld, Cmd, Ar);
	}
	else if (FParse::Command(&Cmd, TEXT("VERSION")))
	{
		rail::IRailGame* const RailGame = RailSdkWrapper::Get().RailGame();
		if (LIKELY(RailGame != nullptr))
		{
			rail::RailString BranchBuildNumber;
			rail::RailResult BranchBuildNumberResult = RailGame->GetBranchBuildNumber(&BranchBuildNumber);
			if (BranchBuildNumberResult == rail::RailResult::kSuccess)
			{
				UE_LOG_ONLINE(Log, TEXT("BranchBuildNumber: %s"), *LexToString(BranchBuildNumber));
			}
			else
			{
				UE_LOG_ONLINE(Log, TEXT("BranchBuildNumber error! %s"), *LexToString(BranchBuildNumberResult));
			}

			rail::RailBranchInfo BranchInfo;
			rail::RailResult BranchInfoResult = RailGame->GetCurrentBranchInfo(&BranchInfo);
			if (BranchInfoResult == rail::RailResult::kSuccess)
			{
				UE_LOG_ONLINE(Log, TEXT("GetCurrentBranchInfo: branch_name=%s"), *LexToString(BranchInfo.branch_name));
				UE_LOG_ONLINE(Log, TEXT("GetCurrentBranchInfo: branch_type=%s"), *LexToString(BranchInfo.branch_type));
				UE_LOG_ONLINE(Log, TEXT("GetCurrentBranchInfo: branch_id=%s"), *LexToString(BranchInfo.branch_id));
				UE_LOG_ONLINE(Log, TEXT("GetCurrentBranchInfo: build_number=%s"), *LexToString(BranchInfo.build_number));
			}
			else
			{
				UE_LOG_ONLINE(Log, TEXT("GetCurrentBranchInfo error! %s"), *LexToString(BranchInfoResult));
			}
		}
	}
	else
	{
		return false;
	}
	return true;
#else
	return false;
#endif
}

bool FOnlineSubsystemTencent::HandleRailSdkWrapperPlayerExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if WITH_TENCENT_RAIL_SDK
	rail::IRailFactory* Factory = RailSdkWrapper::Get().RailFactory();
	if (Factory)
	{
		rail::IRailPlayer* Player = Factory->RailPlayer();
		if (Player)
		{
			if (FParse::Command(&Cmd, TEXT("INFO")))
			{
				const rail::RailID RailId = Player->GetRailID();

				rail::RailString RailPlayerName;
				rail::RailResult RailPlayerNameResult = Player->GetPlayerName(&RailPlayerName);
				const FString PlayerName = LexToString(RailPlayerName);

				const bool bAlreadyLoggedIn = Player->AlreadyLoggedIn();
				static const TCHAR* LoggedInString = TEXT("true (logged in)");
				static const TCHAR* NotLoggedInString = TEXT("false (not logged in)");
				const TCHAR* LoggedInLogString = bAlreadyLoggedIn ? LoggedInString : NotLoggedInString;

				rail::RailString RailDataPath;
				rail::RailResult RailDataPathResult = Player->GetPlayerDataPath(&RailDataPath);
				FString DataPath = LexToString(RailDataPath);

				const rail::EnumRailPlayerOwnershipType OwnershipType = Player->GetPlayerOwnershipType();

				UE_LOG_ONLINE(Display, TEXT("ID: %llu (domain %u)"), RailId.get_id(), static_cast<uint32>(RailId.GetDomain()));
				UE_LOG_ONLINE(Display, TEXT("Name: '%s' (result %s)"), *PlayerName, *LexToString(RailPlayerNameResult));
				UE_LOG_ONLINE(Display, TEXT("Logged in: %s"), LoggedInLogString);
				UE_LOG_ONLINE(Display, TEXT("DataPath: '%s' (result %s)"), *DataPath, *LexToString(RailDataPathResult));
				UE_LOG_ONLINE(Display, TEXT("OwnershipType: %u"), static_cast<uint32>(OwnershipType));
			}
			else if (FParse::Command(&Cmd, TEXT("ACQUIRESESSIONTICKET")))
			{
				const rail::RailID RailId = Player->GetRailID();

				FOnlineAsyncTaskRailAcquireSessionTicket::FCompletionDelegate CompletionDelegate;
				CompletionDelegate.BindLambda([](const FOnlineError& OnlineError, const FString& SessionTicket) {
					if (OnlineError.WasSuccessful())
					{
						UE_LOG_ONLINE(Display, TEXT("SessionTicket: '%s'"), *SessionTicket);
					}
					else
					{
						UE_LOG_ONLINE(Display, TEXT("SessionTicket error: '%s'"), *OnlineError.ToLogString());
					}
				});
				QueueAsyncTask(new FOnlineAsyncTaskRailAcquireSessionTicket(this, RailId, CompletionDelegate));
			}
			else
			{
				return false;
			}
		}
		else
		{
			UE_LOG_ONLINE(Display, TEXT("Unable to get Player"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Display, TEXT("Unable to get Factory"));
	}
	return true;
#else
	return false;
#endif
}

IMessageSanitizerPtr FOnlineSubsystemTencent::GetMessageSanitizer(int32 LocalUserNum, FString& OutAuthTypeToExclude) const
{
#if	WITH_TENCENT_RAIL_SDK
	return TencentMessageSanitizer;
#else
	return nullptr;
#endif
}

FOnlineSubsystemTencent::FOnlineSubsystemTencent(FName InInstanceName) 
	: FOnlineSubsystemImpl(TENCENT_SUBSYSTEM, InInstanceName)
	, OnlineAsyncTaskThreadRunnable(nullptr)
	, OnlineAsyncTaskThread(nullptr)
{
}

FOnlineSubsystemTencent::~FOnlineSubsystemTencent()
{
}

#endif // WITH_TENCENTSDK
