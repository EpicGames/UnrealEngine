// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystem.h"


#if WITH_ENGINE
#endif

#define RAIL_METADATA_KEY_SEPARATOR TEXT(",")

/** pre-pended to all mcp logging */
#undef ONLINE_LOG_PREFIX
#define ONLINE_LOG_PREFIX TEXT("TENCENT: ")
