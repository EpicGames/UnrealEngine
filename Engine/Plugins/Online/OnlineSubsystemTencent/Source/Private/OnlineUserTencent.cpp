// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineUserTencent.h"

#if WITH_TENCENTSDK
#if WITH_TENCENT_RAIL_SDK

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineAsyncTasksTencent.h"
#include "OnlineError.h"
#include "OnlineSubsystemTencent.h"

// FOnlineUserTencent

#define TENCENT_MAX_USERS_PER_QUERY 25

FOnlineUserTencent::FOnlineUserTencent(FOnlineSubsystemTencent* InSubsystem)
	: Subsystem(InSubsystem)
{
}

bool FOnlineUserTencent::QueryUserInfo(int32 LocalUserNum, const TArray<FUniqueNetIdRef>& UserIds)
{
	FOnlineError Error(EOnlineErrorResult::Unknown);

	FUniqueNetIdPtr LocalUserId = Subsystem->GetIdentityInterface()->GetUniquePlayerId(LocalUserNum);
	if (LocalUserId.IsValid())
	{
		StartNextQueryUserInfo(LocalUserNum, LocalUserId.ToSharedRef(), UserIds, 0);
		Error = FOnlineError::Success();
	}
	else
	{
		Error.SetFromErrorCode(FString::Printf(TEXT("Invalid user %d"), LocalUserNum));
	}

	const bool bSucceeded = Error.WasSuccessful();
	if (!bSucceeded)
	{
		UE_LOG_ONLINE_USER(Warning, TEXT("FOnlineUserTencent::QueryUserInfo: %s"), *Error.ToLogString());
		TWeakPtr<FOnlineUserTencent, ESPMode::ThreadSafe> WeakThisPtr(AsShared());
		Subsystem->ExecuteNextTick([WeakThisPtr, LocalUserNum, MovedError = MoveTemp(Error), UserIds]()
		{
			const FOnlineUserTencentPtr This = WeakThisPtr.Pin();
			if (This.IsValid())
			{
				This->TriggerOnQueryUserInfoCompleteDelegates(LocalUserNum, MovedError.WasSuccessful(), UserIds, MovedError.GetErrorCode());
			}
		});
	}
	return bSucceeded;
}

void FOnlineUserTencent::StartNextQueryUserInfo(int32 LocalUserNum, FUniqueNetIdRef LocalUserId, TArray<FUniqueNetIdRef> UserIds, int32 UserIdsQueriedCount)
{
	TArray<FUniqueNetIdRef> UserIdsSubset;
	int32 UserIdx = 0;
	for (; (UserIdsQueriedCount + UserIdx) < UserIds.Num() && UserIdx < TENCENT_MAX_USERS_PER_QUERY; ++UserIdx)
	{
		UserIdsSubset.Emplace(UserIds[UserIdsQueriedCount + UserIdx]);
	}

	FOnOnlineAsyncTaskRailGetUsersInfoComplete CompletionDelegate;
	CompletionDelegate.BindThreadSafeSP(this, &FOnlineUserTencent::QueryUserInfo_Complete, LocalUserNum, LocalUserId, UserIds, UserIdsQueriedCount + UserIdx);
	FOnlineAsyncTaskRailGetUsersInfo* AsyncTask = new FOnlineAsyncTaskRailGetUsersInfo(Subsystem, UserIdsSubset, CompletionDelegate);
	Subsystem->QueueAsyncTask(AsyncTask);
}

void FOnlineUserTencent::QueryUserInfo_Complete(const FGetUsersInfoTaskResult& TaskResult, int32 LocalUserNum, FUniqueNetIdRef LocalUserId, TArray<FUniqueNetIdRef> UserIds, int32 UserIdsQueriedCount)
{
	if (TaskResult.Error.WasSuccessful())
	{
		TArray<FOnlineUserInfoTencentRef>& UsersArray = Users.FindOrAdd(LocalUserId);
		for (const FOnlineUserInfoTencentRef& NewUser : TaskResult.UserInfos)
		{
			const int32 ExistingIndex = UsersArray.IndexOfByPredicate([&NewUser](const FOnlineUserInfoTencentRef& Element) -> bool {
				return *Element->GetUserId() == *NewUser->GetUserId();
			});
			if (ExistingIndex != INDEX_NONE)
			{
				// Overwrite
				UsersArray[ExistingIndex] = NewUser;
			}
			else
			{
				UsersArray.Emplace(NewUser);
			}
		}
	}
	if (TaskResult.Error.WasSuccessful() && UserIdsQueriedCount < UserIds.Num())
	{
		StartNextQueryUserInfo(LocalUserNum, LocalUserId, UserIds, UserIdsQueriedCount);
	}
	else
	{
		TriggerOnQueryUserInfoCompleteDelegates(LocalUserNum, TaskResult.Error.WasSuccessful(), UserIds, TaskResult.Error.GetErrorCode());
	}
}

bool FOnlineUserTencent::GetAllUserInfo(int32 LocalUserNum, TArray<TSharedRef<FOnlineUser>>& OutUsers)
{
	bool bValid = false;
	FUniqueNetIdPtr LocalUserId = Subsystem->GetIdentityInterface()->GetUniquePlayerId(LocalUserNum);
	if (LocalUserId.IsValid())
	{
		TArray<FOnlineUserInfoTencentRef>* UsersForLocalUser = Users.Find(LocalUserId.ToSharedRef());
		if (UsersForLocalUser)
		{
			OutUsers.Reserve(OutUsers.Num() + UsersForLocalUser->Num());
			for (const FOnlineUserInfoTencentRef& UserInfo : *UsersForLocalUser)
			{
				OutUsers.Add(UserInfo);
			}
		}
		bValid = true;
	}
	return bValid;
}

TSharedPtr<FOnlineUser> FOnlineUserTencent::GetUserInfo(int32 LocalUserNum, const FUniqueNetId& UserId)
{
	TSharedPtr<FOnlineUser> Result;
	FUniqueNetIdPtr LocalUserId = Subsystem->GetIdentityInterface()->GetUniquePlayerId(LocalUserNum);
	if (LocalUserId.IsValid())
	{
		TArray<FOnlineUserInfoTencentRef>* UsersForLocalUser = Users.Find(LocalUserId.ToSharedRef());
		if (UsersForLocalUser)
		{
			FOnlineUserInfoTencentRef* FoundUser = UsersForLocalUser->FindByPredicate([&UserId](const FOnlineUserInfoTencentRef& UserInfo) -> bool {
				return *UserInfo->GetUserId() == UserId;
			});
			if (FoundUser)
			{
				Result = *FoundUser;
			}
		}
	}
	return Result;
}

bool FOnlineUserTencent::QueryUserIdMapping(const FUniqueNetId& UserId, const FString& DisplayName, const FOnQueryUserMappingComplete& CompletionDelegate)
{
	// Not implemented
	return false;
}

bool FOnlineUserTencent::QueryExternalIdMappings(const FUniqueNetId& LocalUserId, const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, const FOnQueryExternalIdMappingsComplete& Delegate)
{
	// Not implemented
	return false;
}

void FOnlineUserTencent::GetExternalIdMappings(const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, TArray<FUniqueNetIdPtr>& OutIds)
{
	// Not implemented, just set OutIds to be an array of size ExternalIds but with empty shared pointers
	OutIds.SetNum(ExternalIds.Num());
}

FUniqueNetIdPtr FOnlineUserTencent::GetExternalIdMapping(const FExternalIdQueryOptions& QueryOptions, const FString& ExternalId)
{
	// Not implemented
	FUniqueNetIdPtr Result;
	return Result;
}

#endif // WITH_TENCENT_RAIL_SDK
#endif // WITH_TENCENTSDK
