// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TENCENTSDK
#if WITH_TENCENT_RAIL_SDK

#include "Interfaces/OnlineUserInterface.h"
#include "OnlineSubsystemTencentTypes.h"
#include "OnlineSubsystemTencentPackage.h"

class FOnlineSubsystemTencent;
struct FGetUsersInfoTaskResult;

class FOnlineUserInfoTencent : public FOnlineUser
{
public:
	FOnlineUserInfoTencent() = delete;
	FOnlineUserInfoTencent(const FUniqueNetIdRef& InUserId) : UserId(InUserId) {}
	virtual ~FOnlineUserInfoTencent() = default;

	// FOnlineUser
	virtual FUniqueNetIdRef GetUserId() const override { return UserId; }
	virtual FString GetRealName() const override
	{
		FString RealName;
		GetUserAttribute(USER_ATTR_REALNAME, RealName);
		return RealName;
	}
	virtual FString GetDisplayName(const FString& Platform = FString()) const override
	{
		FString DisplayName;
		GetUserAttribute(USER_ATTR_DISPLAYNAME, DisplayName);
		return DisplayName;
	}
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override
	{
		const FString* AttributeVal = UserAttributes.Find(AttrName);
		if (AttributeVal)
		{
			OutAttrValue = *AttributeVal;
		}
		return AttributeVal != nullptr;
	}
	virtual bool SetUserLocalAttribute(const FString& AttrName, const FString& InAttrValue) override
	{
		UserAttributes.Emplace(AttrName, InAttrValue);
		return true;
	}

	// FOnlineUserInfoTencent

PACKAGE_SCOPE:
	/**
	 * Set an attribute
	 * @param AttrName name of the attribute
	 * @param AttrVal value of the attribute
	 */
	void SetUserAttribute(FString&& AttrName, FString&& AttrVal)
	{
		UserAttributes.Emplace(MoveTemp(AttrName), MoveTemp(AttrVal));
	}

protected:
	/** The user's id */
	FUniqueNetIdRef UserId;
	/** User attributes */
	TMap<FString, FString> UserAttributes;
};

/** Shared reference to user info */
typedef TSharedRef<FOnlineUserInfoTencent> FOnlineUserInfoTencentRef;

/** Map of local user to online user infos */
typedef TUniqueNetIdMap<TArray<FOnlineUserInfoTencentRef>> FOnlineUserInfoTencentMap;

/**
 *	Tencent service implementation of the online user interface
 */
class FOnlineUserTencent : 
	public IOnlineUser,
	public TSharedFromThis<FOnlineUserTencent, ESPMode::ThreadSafe>
{
public:
	FOnlineUserTencent() = delete;
	FOnlineUserTencent(FOnlineSubsystemTencent* InSubsystem);
	virtual ~FOnlineUserTencent() = default;

	//~ Begin IOnlineUser interface
	virtual bool QueryUserInfo(int32 LocalUserNum, const TArray<FUniqueNetIdRef >& UserIds) override;
	virtual bool GetAllUserInfo(int32 LocalUserNum, TArray< TSharedRef<FOnlineUser> >& OutUsers) override;
	virtual TSharedPtr<FOnlineUser> GetUserInfo(int32 LocalUserNum, const FUniqueNetId& UserId) override;
	virtual bool QueryUserIdMapping(const FUniqueNetId& UserId, const FString& DisplayNameOrEmail, const FOnQueryUserMappingComplete& Delegate = FOnQueryUserMappingComplete()) override;
	virtual bool QueryExternalIdMappings(const FUniqueNetId& UserId, const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, const FOnQueryExternalIdMappingsComplete& Delegate = FOnQueryExternalIdMappingsComplete()) override;
	virtual void GetExternalIdMappings(const FExternalIdQueryOptions& QueryOptions, const TArray<FString>& ExternalIds, TArray<FUniqueNetIdPtr>& OutIds) override;
	virtual FUniqueNetIdPtr GetExternalIdMapping(const FExternalIdQueryOptions& QueryOptions, const FString& ExternalId) override;
	//~ End IOnlineUser interface

private:
	void StartNextQueryUserInfo(int32 LocalUserNum, FUniqueNetIdRef LocalUserId, TArray<FUniqueNetIdRef> UserIds, int32 UserIdsQueriedCount);
	void QueryUserInfo_Complete(const FGetUsersInfoTaskResult& TaskResult, int32 LocalUserNum, FUniqueNetIdRef LocalUserId, TArray<FUniqueNetIdRef> UserIds, int32 UserIdsQueriedCount);

private:
	/** Owning subsystem */
	FOnlineSubsystemTencent* Subsystem;

	/** Users we have retrieved data for */
	FOnlineUserInfoTencentMap Users;
};

typedef TSharedPtr<FOnlineUserTencent, ESPMode::ThreadSafe> FOnlineUserTencentPtr;

#endif // WITH_TENCENT_RAIL_SDK
#endif // WITH_TENCENTSDK
