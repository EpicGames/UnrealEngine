// Copyright Epic Games, Inc. All Rights Reserved.

#include "RailSDK.h"

#if WITH_TENCENTSDK
#if WITH_TENCENT_RAIL_SDK

#include "Containers/StringConv.h"
#include "Containers/UnrealString.h"
#include "RailSdkWrapper.h"

FString LexToString(const rail::RailSystemState State)
{
	switch (State)
	{
		case rail::RailSystemState::kSystemStateUnknown:
			return TEXT("kSystemStateUnknown");

		case rail::RailSystemState::kSystemStatePlatformOnline:
			return TEXT("kSystemStatePlatformOnline");
		case rail::RailSystemState::kSystemStatePlatformOffline:
			return TEXT("kSystemStatePlatformOffline");
		case rail::RailSystemState::kSystemStatePlatformExit:
			return TEXT("kSystemStatePlatformExit");

		case rail::RailSystemState::kSystemStatePlayerOwnershipExpired:
			return TEXT("kSystemStatePlayerOwnershipExpired");
		case rail::RailSystemState::kSystemStatePlayerOwnershipActivated:
			return TEXT("kSystemStatePlayerOwnershipActivated");
	}
	return FString::Printf(TEXT("Invalid: %u"), static_cast<uint32>(State));
}

FString LexToString(const rail::RailResult InResult)
{
	rail::RailString ErrorStr;
	if (rail::IRailUtils* RailUtils = RailSdkWrapper::Get().RailUtils())
	{
		RailUtils->GetErrorString(InResult, &ErrorStr);
	}

	return LexToString(ErrorStr);
}

FString LexToString(const rail::EnumRailPlayerOnLineState PlayerOnlineState)
{
	switch (PlayerOnlineState)
	{
	case rail::EnumRailPlayerOnLineState::kRailOnlineStateUnknown:
		return TEXT("Unknown");
	case rail::EnumRailPlayerOnLineState::kRailOnlineStateOffLine:
		return TEXT("Offline");
	case rail::EnumRailPlayerOnLineState::kRailOnlineStateOnLine:
		return TEXT("Online");
	case rail::EnumRailPlayerOnLineState::kRailOnlineStateBusy:
		return TEXT("Busy");
	case rail::EnumRailPlayerOnLineState::kRailOnlineStateLeave:
		return TEXT("Leave");
	case rail::EnumRailPlayerOnLineState::kRailOnlineStateGameDefinePlayingState:
		return TEXT("GameDefinePlayingState");
	}
	return FString::Printf(TEXT("(Invalid:%d)"), static_cast<int32>(PlayerOnlineState));
}

FString LexToString(const rail::EnumRailUsersInviteType InviteType)
{
	switch (InviteType)
	{
		case rail::EnumRailUsersInviteType::kRailUsersInviteTypeGame:
			return TEXT("InviteGame");
		case rail::EnumRailUsersInviteType::kRailUsersInviteTypeRoom:
			return TEXT("InviteRoom");
	}
	return FString::Printf(TEXT("(Invalid:%d)"), static_cast<int32>(InviteType));
}

FString LexToString(const rail::EnumRailUsersInviteResponseType ResponseType)
{
	switch (ResponseType)
	{
		case rail::EnumRailUsersInviteResponseType::kRailInviteResponseTypeUnknown:
			return TEXT("ResponseUnknown");
		case rail::EnumRailUsersInviteResponseType::kRailInviteResponseTypeAccepted:
			return TEXT("ResponseAccepted");
		case rail::EnumRailUsersInviteResponseType::kRailInviteResponseTypeRejected:
			return TEXT("ResponseRejected");
		case rail::EnumRailUsersInviteResponseType::kRailInviteResponseTypeIgnore:
			return TEXT("ResponseIgnore");
		case rail::EnumRailUsersInviteResponseType::kRailInviteResponseTypeTimeout:
			return TEXT("ResponseTimeout");
	}
	return FString::Printf(TEXT("(Invalid:%d)"), static_cast<int32>(ResponseType));
}

FString LexToString(const rail::RailFriendPlayedGamePlayState PlayState)
{
	switch (PlayState)
	{
		case rail::RailFriendPlayedGamePlayState::kRailFriendPlayedGamePlayStatePlaying:
			return TEXT("Playing");
		case rail::RailFriendPlayedGamePlayState::kRailFriendPlayedGamePlayStatePlayed:
			return TEXT("Played");
	}
	return FString::Printf(TEXT("(Invalid:%d)"), static_cast<int32>(PlayState));
}

FString LexToString(const rail::EnumRailAssetState AssetState)
{
	switch (AssetState)
	{
		case rail::EnumRailAssetState::kRailAssetStateNormal:
			return TEXT("Normal");
		case rail::EnumRailAssetState::kRailAssetStateInConsume:
			return TEXT("Consumed");
	}
	return FString::Printf(TEXT("(Invalid:%d)"), static_cast<int32>(AssetState));
}

FString LexToString(const rail::RailString& RailString)
{
	FUTF8ToTCHAR Converter(RailString.c_str(), RailString.size());
	return FString::ConstructFromPtrSize(Converter.Get(), Converter.Length());
}

void ToRailString(const FString& Str, rail::RailString& OutString)
{
	FTCHARToUTF8 Converter(*Str, Str.Len());
	OutString.assign(Converter.Get(), Converter.Length());
}

#endif // WITH_TENCENT_RAIL_SDK
#endif // WITH_TENCENTSDK
