// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TENCENTSDK
#if WITH_TENCENT_RAIL_SDK

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS  /* unsafe conversion from 'FARPROC' to 'rail::helper::*' */

THIRD_PARTY_INCLUDES_START
#include "rail/sdk/rail_api.h"
#include "rail/sdk/rail_event.h"
THIRD_PARTY_INCLUDES_END

PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

FString LexToString(const rail::RailSystemState State);
FString LexToString(const rail::RailResult Result);
FString LexToString(const rail::EnumRailPlayerOnLineState PlayerOnlineState);
FString LexToString(const rail::EnumRailUsersInviteType InviteType);
FString LexToString(const rail::EnumRailUsersInviteResponseType ResponseType);
FString LexToString(const rail::RailFriendPlayedGamePlayState PlayState);
FString LexToString(const rail::EnumRailAssetState AssetState);

FString LexToString(const rail::RailString& RailString);

void ToRailString(const FString& Str, rail::RailString& OutString);

#endif // WITH_TENCENT_RAIL_SDK
#endif // WITH_TENCENTSDK
