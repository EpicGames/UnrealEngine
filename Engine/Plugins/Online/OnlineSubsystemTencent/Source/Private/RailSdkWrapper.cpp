// Copyright Epic Games, Inc. All Rights Reserved.

#include "RailSdkWrapper.h"
#include "HAL/Platform.h"

#if PLATFORM_WINDOWS
#include "Misc/CommandLine.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include <ShellAPI.h>
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#if WITH_TENCENTSDK
#if WITH_TENCENT_RAIL_SDK
#include "Misc/Paths.h"
#include "OnlineSubsystem.h"

#define RAIL_SDK_MODULE_NAME	TEXT("rail_api64.dll")

RailSdkWrapper::RailSdkWrapper()
	: bIsInitialized(false)
{
}

RailSdkWrapper::~RailSdkWrapper()
{
	Shutdown();
}

RailSdkWrapper& RailSdkWrapper::Get()
{
	static RailSdkWrapper Singleton;
	return Singleton;
}

bool RailSdkWrapper::Load()
{
	if (!RailSdkDll.IsLoaded())
	{
		RailSdkDll.Load(
			FPaths::Combine(*FPaths::ProjectDir(), TEXT("Binaries/ThirdParty/Tencent/")),
			RAIL_SDK_MODULE_NAME
		);
	}
	else
	{
		UE_LOG_ONLINE(Log, TEXT("Rail SDK is already loaded"));
	}
	return RailSdkDll.IsLoaded();
}

void RailSdkWrapper::Shutdown()
{
	// Unload dll only on shutdown
	RailSdkDll.Unload();
}

// get old school command line
static bool GetLaunchArgs(int32& argc, ANSICHAR**& argv)
{
#if PLATFORM_WINDOWS
	LPWSTR* szArglist = ::CommandLineToArgvW(FCommandLine::Get(), &argc);
	if (szArglist != nullptr)
	{
		argv = new ANSICHAR*[argc];
		for (int i = 0; i < argc; i++)
		{
			FTCHARToUTF8 UTF8LanguageString(szArglist[i]);
			argv[i] = new ANSICHAR[UTF8LanguageString.Length() + 1];
			FMemory::Memcpy((void*)argv[i], UTF8LanguageString.Get(), UTF8LanguageString.Length() + 1);
		}
		return true;
	}
#endif //PLATFORM_WINDOWS
	return false;
}

// free old school command line
static void FreeLaunchArgs(int32 argc, ANSICHAR** argv)
{
	for (int i = 0; i < argc; i++)
	{
		delete[] argv[i];
	}
	delete[] argv;
}

bool RailSdkWrapper::RailNeedRestartAppForCheckingEnvironment(rail::RailGameID game_id)
{
	bool bResult = false;
	if (RailSdkDll.IsLoaded())
	{
		int32 argc = 0;
		ANSICHAR** argv = nullptr;
		GetLaunchArgs(argc, argv);

		bResult = rail::helper::Invoker((HMODULE)RailSdkDll.GetDllHandle())
			.RailNeedRestartAppForCheckingEnvironment(game_id, argc, (const char**)argv);

		FreeLaunchArgs(argc, argv);
	}
	return bResult;
}

bool RailSdkWrapper::RailInitialize()
{
	if (!bIsInitialized)
	{
		if (RailSdkDll.IsLoaded())
		{
			bIsInitialized = rail::helper::Invoker((HMODULE)RailSdkDll.GetDllHandle())
				.RailInitialize();
		}
	}
	return bIsInitialized;
}

void RailSdkWrapper::RailFireEvents()
{
	if (RailSdkDll.IsLoaded())
	{
		rail::helper::Invoker((HMODULE)RailSdkDll.GetDllHandle())
			.RailFireEvents();
	}
}

void RailSdkWrapper::RailFinalize()
{
	if (RailSdkDll.IsLoaded())
	{
		rail::helper::Invoker((HMODULE)RailSdkDll.GetDllHandle())
			.RailFinalize();
	}
}

rail::IRailFactory* RailSdkWrapper::RailFactory() const
{
	if (RailSdkDll.IsLoaded())
	{
		return rail::helper::Invoker((HMODULE)RailSdkDll.GetDllHandle())
			.RailFactory();
	}

	return nullptr;
}

rail::IRailUtils* RailSdkWrapper::RailUtils() const
{
	rail::IRailUtils* Utils = nullptr;

	if (IsInitialized())
	{
		if (rail::IRailFactory* const RailFactory = RailSdkWrapper::RailFactory())
		{
			Utils = RailFactory->RailUtils();
		}
		else
		{
			UE_LOG_ONLINE(Log, TEXT("No IRailFactory found!"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Log, TEXT("RailSDK not initialized!"));
	}

	if (!Utils)
	{
		UE_LOG_ONLINE(Log, TEXT("No IRailUtils found!"));
	}

	return Utils;
}

rail::IRailFriends* RailSdkWrapper::RailFriends() const
{
	rail::IRailFriends* Friends = nullptr;

	if (IsInitialized())
	{
		if (rail::IRailFactory* const RailFactory = RailSdkWrapper::RailFactory())
		{
			Friends = RailFactory->RailFriends();
		}
		else
		{
			UE_LOG_ONLINE(Log, TEXT("No IRailFactory found!"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Log, TEXT("RailSDK not initialized!"));
	}

	if (!Friends)
	{
		UE_LOG_ONLINE(Log, TEXT("No IRailFriends found!"));
	}

	return Friends;
}

rail::IRailPlayer* RailSdkWrapper::RailPlayer() const
{
	rail::IRailPlayer* Player = nullptr;

	if (IsInitialized())
	{
		if (rail::IRailFactory* const RailFactory = RailSdkWrapper::RailFactory())
		{
			Player = RailFactory->RailPlayer();
		}
		else
		{
			UE_LOG_ONLINE(Log, TEXT("No IRailFactory found!"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Log, TEXT("RailSDK not initialized!"));
	}

	if (!Player)
	{
		UE_LOG_ONLINE(Log, TEXT("No IRailPlayer found!"));
	}

	return Player;
}

rail::IRailUsersHelper* RailSdkWrapper::RailUsersHelper() const
{
	rail::IRailUsersHelper* Users = nullptr;

	if (IsInitialized())
	{
		if (rail::IRailFactory* const RailFactory = RailSdkWrapper::RailFactory())
		{
			Users = RailFactory->RailUsersHelper();
		}
		else
		{
			UE_LOG_ONLINE(Log, TEXT("No IRailFactory found!"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Log, TEXT("RailSDK not initialized!"));
	}

	if (!Users)
	{
		UE_LOG_ONLINE(Log, TEXT("No IRailUsersHelper found!"));
	}

	return Users;
}

rail::IRailInGamePurchase* RailSdkWrapper::RailInGamePurchase() const
{
	rail::IRailInGamePurchase* InGamePurchase = nullptr;

	if (IsInitialized())
	{
		if (rail::IRailFactory* const RailFactory = RailSdkWrapper::RailFactory())
		{
			InGamePurchase = RailFactory->RailInGamePurchase();
		}
		else
		{
			UE_LOG_ONLINE(Log, TEXT("No IRailFactory found!"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Log, TEXT("RailSDK not initialized!"));
	}

	if (!InGamePurchase)
	{
		UE_LOG_ONLINE(Log, TEXT("No IRailInGamePurchase found!"));
	}

	return InGamePurchase;
}

rail::IRailAssets* RailSdkWrapper::RailAssets() const
{
	rail::IRailAssets* Assets = nullptr;

	if (IsInitialized())
	{
		if (rail::IRailFactory* const RailFactory = RailSdkWrapper::RailFactory())
		{
			if (rail::IRailAssetsHelper* const AssetsHelper = RailFactory->RailAssetsHelper())
			{
				Assets = AssetsHelper->OpenAssets();
			}
			else
			{
				UE_LOG_ONLINE(Log, TEXT("No IRailAssetsHelper found!"));
			}
		}
		else
		{
			UE_LOG_ONLINE(Log, TEXT("No IRailFactory found!"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Log, TEXT("RailSDK not initialized!"));
	}

	if (!Assets)
	{
		UE_LOG_ONLINE(Log, TEXT("No IRailAssets found!"));
	}

	return Assets;
}

rail::IRailGame* RailSdkWrapper::RailGame() const
{
	rail::IRailGame* Game = nullptr;

	if (IsInitialized())
	{
		if (rail::IRailFactory* const RailFactory = RailSdkWrapper::RailFactory())
		{
			Game = RailFactory->RailGame();
		}
		else
		{
			UE_LOG_ONLINE(Log, TEXT("No IRailFactory found!"));
		}
	}
	else
	{
		UE_LOG_ONLINE(Log, TEXT("RailSDK not initialized!"));
	}

	if (!Game)
	{
		UE_LOG_ONLINE(Log, TEXT("No IRailGame found!"));
	}

	return Game;
}

void RailSdkWrapper::RailRegisterEvent(rail::RAIL_EVENT_ID event_id, rail::IRailEvent* event_handler)
{
	if (RailSdkDll.IsLoaded())
	{
		rail::helper::Invoker((HMODULE)RailSdkDll.GetDllHandle())
			.RailRegisterEvent(event_id, event_handler);
	}
}

void RailSdkWrapper::RailUnregisterEvent(rail::RAIL_EVENT_ID event_id, rail::IRailEvent* event_handler)
{
	if (RailSdkDll.IsLoaded())
	{
		rail::helper::Invoker((HMODULE)RailSdkDll.GetDllHandle())
			.RailUnregisterEvent(event_id, event_handler);
	}
}

#endif // WITH_TENCENT_RAIL_SDK
#endif // WITH_TENCENTSDK
