// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TENCENTSDK
#if WITH_TENCENT_RAIL_SDK

#include "RailSDK.h"
#include "TencentDllMgr.h"

/**
 * Singleton wrapper to manage the Rail SDK usage
 */
class RailSdkWrapper
{
public:
	virtual ~RailSdkWrapper();
	static RailSdkWrapper& Get();

	// Wrappers for main entry points for Rail SDK
	bool RailNeedRestartAppForCheckingEnvironment(rail::RailGameID game_id);
	bool RailInitialize();
	void RailFireEvents();
	void RailFinalize();
	void RailRegisterEvent(rail::RAIL_EVENT_ID event_id, rail::IRailEvent* event_handler);
	void RailUnregisterEvent(rail::RAIL_EVENT_ID event_id, rail::IRailEvent* event_handler);

	bool IsInitialized() const { return bIsInitialized; }

	rail::IRailFactory* RailFactory() const;
	rail::IRailFriends* RailFriends() const;
	rail::IRailPlayer* RailPlayer() const;
	rail::IRailUtils* RailUtils() const;
	rail::IRailUsersHelper* RailUsersHelper() const;
	rail::IRailInGamePurchase* RailInGamePurchase() const;
	rail::IRailAssets* RailAssets() const;
	rail::IRailGame* RailGame() const;

private:
	/** Singleton access only */
	RailSdkWrapper();

	/** Load the Rail SDK dlls */
	bool Load();
	/** Unload the Rail SDK dlls */
	void Shutdown();

	/** Is the DLL loaded and RailInitialize called */
	bool bIsInitialized;
	/** Dll loaded on init and unloaded and shutdown */
	FTencentDll RailSdkDll;

	friend class FOnlineSubsystemTencent;
};

#endif // WITH_TENCENT_TCLS
#endif // WITH_TENCENTSDK
