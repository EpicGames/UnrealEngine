// Copyright Epic Games, Inc. All Rights Reserved.

#include "TencentDllMgr.h"
#include "Misc/Paths.h"
#include "OnlineSubsystem.h"

FTencentDll::FTencentDll()
	: DllHandle(nullptr)
{
}

FTencentDll::~FTencentDll()
{
	Unload();
}

void* FTencentDll::Load(const FString& DllPath, const FString& DllFile)
{
	FString TencentSdkDllPath = DllPath;

#if PLATFORM_WINDOWS
	TencentSdkDllPath = FPaths::Combine(*TencentSdkDllPath, TEXT("Win64"));
#elif PLATFORM_LINUX
	TencentSdkDllPath = FPaths::Combine(*TencentSdkDllPath, TEXT("linux64"));
#endif

	DllHandle = FPlatformProcess::GetDllHandle(*FPaths::Combine(*TencentSdkDllPath, *DllFile));
	if (DllHandle == nullptr)
	{
		UE_LOG_ONLINE(Warning, TEXT("Could not load Tencent dll: %s in %s"), *DllFile, *TencentSdkDllPath);
	}
	return DllHandle;
}

void FTencentDll::Unload()
{
	if (DllHandle != nullptr)
	{
		FPlatformProcess::FreeDllHandle(DllHandle);
		DllHandle = nullptr;
	}
}
