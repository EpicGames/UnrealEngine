// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


class FString;

/**
 * Singleton wrapper to manage the Tcls proxy connection
 */
class FTencentDll
{
public:
	FTencentDll();
	virtual ~FTencentDll();
	void* Load(const FString& DllPath, const FString& DllFile);
	void Unload();
	inline bool IsLoaded() const { return DllHandle != nullptr; }
	inline void* GetDllHandle() const { return DllHandle; }

private:
	void* DllHandle;
};

