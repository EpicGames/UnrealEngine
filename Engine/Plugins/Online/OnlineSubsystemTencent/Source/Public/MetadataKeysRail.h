// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TENCENT_RAIL_SDK

/**
 * Used by rich presence
 */
/** List of all keys that make up a user's presence (FString) */
#define RAIL_PRESENCE_PRESENCE_KEYS TEXT("PresenceKeys")
/** Presence status message (FString) */
#define RAIL_PRESENCE_STATUS_KEY TEXT("Status")
/** Running AppId for the user (FString) */
#define RAIL_PRESENCE_APPID_KEY TEXT("AppId")
/** GameSession session id, if present (FString) */
#define RAIL_PRESENCE_SESSION_ID_KEY TEXT("PresenceSessionId")
/** Basic presence data packed into a bitmask (uint32) */
#define RAIL_PRESENCE_PRESENCEBITS_KEY TEXT("PresenceBits")


/**
 * Used to convey session presence session
 */
/** Presence Session id (FString) */
#define RAIL_SESSION_ID_KEY TEXT("SessionId")
/** Owning user of the presence session (uint64) */
#define RAIL_SESSION_OWNING_USER_ID_KEY TEXT("OwningUserId")
/** Bits representing various flags on the session (uint32) */
#define RAIL_SESSION_SESSIONBITS_KEY TEXT("SessionBits")
/** Build Id for the running application (int32) */
#define RAIL_SESSION_BUILDUNIQUEID_KEY TEXT("BuildUniqueId")


#endif // WITH_TENCENT_RAIL_SDK
