// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TENCENTSDK

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystemPackage.h"
#include "Serialization/JsonSerializerMacros.h"

class FOnlineSubsystemTencent;

/**
 * Info associated with a user account generated by signing in to Tencent
 */
class ONLINESUBSYSTEMTENCENT_API FUserOnlineAccountTencent :
	public FUserOnlineAccount
{
public:

	// FOnlineUser
	
	virtual FUniqueNetIdRef GetUserId() const override;
	virtual FString GetRealName() const override;
	virtual FString GetDisplayName(const FString& Platform = FString()) const override;
	virtual bool GetUserAttribute(const FString& AttrName, FString& OutAttrValue) const override;
	virtual bool SetUserAttribute(const FString& AttrName, const FString& AttrValue) override;

	// FUserOnlineAccount

	virtual FString GetAccessToken() const override;
	virtual bool GetAuthAttribute(const FString& AttrName, FString& OutAttrValue) const override;

	/**
	 * Get account data attribute
	 *
	 * @param Key account data entry key
	 * @param OutVal [out] value that was found
	 *
	 * @return true if entry was found
	 */
	inline bool GetAccountData(const FString& Key, FString& OutVal) const
	{
		const FString* FoundVal = AccountData.Find(Key);
		if (FoundVal != NULL)
		{
			OutVal = *FoundVal;
			return true;
		}
		return false;
	}

	/**
	 * Set account data attribute
	 *
	 * @param Key account data entry key
	 * @param Val value that we want to set
	 *
	 * @return true if entry changed
	 */
	inline bool SetAccountData(const FString& Key, const FString& Val)
	{
		const FString* FoundVal = AccountData.Find(Key);
		if (FoundVal == NULL || *FoundVal != Val)
		{
			AccountData.Add(Key, Val);
			return true;
		}
		return false;
	}

	// FUserOnlineAccountTencent

	/**
	 * Init/default constructor
	 */
	FUserOnlineAccountTencent(const FUniqueNetIdRef InUserId)
		: UserId(InUserId)
	{ }

	/**
	 * Destructor
	 */
	virtual ~FUserOnlineAccountTencent() = default;

	/** User Id represented as a FUniqueNetId */
	FUniqueNetIdRef UserId;
	/** Any addition account data associated with the user */
	FJsonSerializableKeyValueMap AccountData;
	/** Full auth code which can be exchanged for access */
	FString AuthToken;
};

/**
 * Tencent service implementation of the online identity interface
 */
class ONLINESUBSYSTEMTENCENT_API FOnlineIdentityTencent :
	public IOnlineIdentity,
	public TSharedFromThis<FOnlineIdentityTencent, ESPMode::ThreadSafe>
{
public:

	// IOnlineIdentity

	virtual bool Login(int32 LocalUserNum, const FOnlineAccountCredentials& AccountCredentials) override;
	virtual bool Logout(int32 LocalUserNum) override;
	virtual bool AutoLogin(int32 LocalUserNum) override;
	virtual TSharedPtr<FUserOnlineAccount> GetUserAccount(const FUniqueNetId& UserId) const override;
	virtual TArray<TSharedPtr<FUserOnlineAccount> > GetAllUserAccounts() const override;
	virtual FUniqueNetIdPtr GetUniquePlayerId(int32 LocalUserNum) const override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(uint8* Bytes, int32 Size) override;
	virtual FUniqueNetIdPtr CreateUniquePlayerId(const FString& Str) override;
	virtual ELoginStatus::Type GetLoginStatus(int32 LocalUserNum) const override;
	virtual ELoginStatus::Type GetLoginStatus(const FUniqueNetId& UserId) const override;
	virtual FString GetPlayerNickname(int32 LocalUserNum) const override;
	virtual FString GetPlayerNickname(const FUniqueNetId& UserId) const override;
	virtual FString GetAuthToken(int32 LocalUserNum) const override;
	virtual void GetUserPrivilege(const FUniqueNetId& UserId, EUserPrivileges::Type Privilege, const FOnGetUserPrivilegeCompleteDelegate& Delegate, EShowPrivilegeResolveUI ShowResolveUI=EShowPrivilegeResolveUI::Default) override;
	virtual FPlatformUserId GetPlatformUserIdFromUniqueNetId(const FUniqueNetId& UniqueNetId) const override;
	virtual FString GetAuthType() const override;
	virtual void RevokeAuthToken(const FUniqueNetId& UserId, const FOnRevokeAuthTokenCompleteDelegate& Delegate) override;

PACKAGE_SCOPE:

	// FOnlineIdentityTencent

	/**
	 * Constructor
	 *
	 * @param InSubsystem Tencent subsystem being used
	 */
	FOnlineIdentityTencent(FOnlineSubsystemTencent* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlineIdentityTencent();

	TSharedPtr<FUserOnlineAccountTencent> GetUserAccountTencent(const FUniqueNetId& UserId) const;

	/**
	 * Get the local user index for a given user id
	 *
	 * @param UserId id of logged in user
	 * @param OutLocalIdx [out] index of local user
	 *
	 * @return false if user was not found
	 */
	bool GetLocalUserIdx(const FUniqueNetId& UserId, int32& OutLocalIdx) const;

private:
#if WITH_TENCENT_RAIL_SDK
	void OnRailAcquireSessionTicket(const FOnlineError& OnlineError, const FString& SessionTicket, int32 LocalUserNum, const bool bIsNewUser);
#endif

	/**
	 * Must use the initialization constructor instead
	 */
	FOnlineIdentityTencent() = delete;
	
	/** Users that have been registered. Only one Tencent user can ever be logged in. */
	TSharedPtr<FUserOnlineAccountTencent> UserAccount;
	/** Ids mapped to locally registered users */
	TMap<int32, FUniqueNetIdPtr > UserIds;

	/** Reference to the main subsystem */
	FOnlineSubsystemTencent* Subsystem;
};

typedef TSharedPtr<FOnlineIdentityTencent, ESPMode::ThreadSafe> FOnlineIdentityTencentPtr;

#endif // WITH_TENCENTSDK
