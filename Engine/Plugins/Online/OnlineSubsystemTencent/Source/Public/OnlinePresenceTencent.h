// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#if WITH_TENCENTSDK
#if WITH_TENCENT_RAIL_SDK

#include "OnlineAsyncTasksTencent.h"

class FOnlineSubsystemTencent;

/**
 * Presence data for a single client logged in by the user
 */
class FOnlineUserPresenceTencent :
	public FOnlineUserPresence

{
public:

};

typedef TUniqueNetIdMap<TSharedRef<FOnlineUserPresenceTencent>> FOnlineUserPresenceTencentMap;

/**
 * Tencent/Rail service implementation of the online presence interface
 */
class FOnlinePresenceTencent :
	public IOnlinePresence,
	public TSharedFromThis<FOnlinePresenceTencent, ESPMode::ThreadSafe>
{
public:

	//~ Begin FOnlinePresence Interface
	virtual void SetPresence(const FUniqueNetId& User, const FOnlineUserPresenceStatus& Status, const FOnPresenceTaskCompleteDelegate& Delegate = FOnPresenceTaskCompleteDelegate()) override;
	virtual void QueryPresence(const FUniqueNetId& User, const FOnPresenceTaskCompleteDelegate& Delegate = FOnPresenceTaskCompleteDelegate()) override;
	virtual EOnlineCachedResult::Type GetCachedPresence(const FUniqueNetId& User, TSharedPtr<FOnlineUserPresence>& OutPresence) override;
	virtual EOnlineCachedResult::Type GetCachedPresenceForApp(const FUniqueNetId& LocalUserId, const FUniqueNetId& User, const FString& AppId, TSharedPtr<FOnlineUserPresence>& OutPresence) override;
	//~ End FOnlinePresence Interface

	/**
	 * Constructor
	 *
	 * @param InSubsystem mcp subsystem being used
	 */
	FOnlinePresenceTencent(FOnlineSubsystemTencent* InSubsystem);

	/**
	 * Destructor
	 */
	virtual ~FOnlinePresenceTencent();

	/**
	 * Initialize the interface
	 *
	 * @return true if successful, false otherwise
	 */
	bool Init();

	/**
	 * Shutdown the presence interface
	 */
	void Shutdown();

PACKAGE_SCOPE:

	/**
	 * Set the current online state for a given user based on information from the Rail SDK
	 *
	 * @param UserId user whose online state has changed
	 * @param NewState current online state of the user
	 */
	void SetUserOnlineState(const FUniqueNetId& UserId, EOnlinePresenceState::Type NewState);

	/** Update session related presence keys */
	void UpdatePresenceFromSessionData();

private:
	
	FOnlinePresenceTencent() = delete;

	/** Reference to the existing online subsystem */
	FOnlineSubsystemTencent* TencentSubsystem;
	/** Cache of all user presence data received/sent */
	FOnlineUserPresenceTencentMap CachedPresence;
	/** Cache of online state of various users (in presence or otherwise) */
	TUniqueNetIdMap<EOnlinePresenceState::Type> UserOnlineStatus;
	/** Handle to login change events for clearing presence */
	FDelegateHandle OnLoginChangedHandle;
	/** Handle to friend metadata change events */
	FDelegateHandle OnFriendMetadataChangedDelegateHandle;

	/**
	 * Fill in the presence data with information 
	 * from the active game session, if present
	 *
	 * @param InPresenceStatus presence to fill with data about game session
	 */
	void GetGameSessionPresenceData(FMetadataPropertiesRail& InPresenceStatus);

	/**
	 * Notification that login status has changed for a user
	 *
	 * @param LocalUserNum id of local user whose login status has changed
	 */
	void OnLoginChanged(int32 LocalUserNum);

	/** Delegate fired when platform says friend metadata has changed */
	void OnFriendMetadataChangedEvent(const FUniqueNetId& UserId, const FMetadataPropertiesRail& Metadata);
};

typedef TSharedPtr<FOnlinePresenceTencent, ESPMode::ThreadSafe> FOnlinePresenceTencentPtr;

inline EOnlinePresenceState::Type RailOnlineStateToOnlinePresence(rail::EnumRailPlayerOnLineState RailState)
{
	switch (RailState)
	{
		case rail::kRailOnlineStateOffLine:  // player is off-line.
			return EOnlinePresenceState::Type::Offline;
		case rail::kRailOnlineStateOnLine:  // player is on-line.
			return EOnlinePresenceState::Type::Online;
		case rail::kRailOnlineStateBusy: // player is on-line, but busy.
			return EOnlinePresenceState::Type::DoNotDisturb;
		case rail::kRailOnlineStateLeave:  // player is auto away.
			return EOnlinePresenceState::Type::Away;
		case rail::kRailOnlineStateGameDefinePlayingState:  // player is in the game define playing state
			return EOnlinePresenceState::Type::Online;
		case rail::kRailOnlineStateUnknown:
		default:
			return EOnlinePresenceState::Type::Offline;
	}
}

#endif // WITH_TENCENT_RAIL_SDK
#endif // WITH_TENCENTSDK
