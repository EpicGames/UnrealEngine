// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#if WITH_TENCENT_RAIL_SDK

#include "OnlineSessionTencent.h"
#include "OnlineSubsystemTencentTypes.h"
#include "OnlineSubsystemTencentPackage.h"

class FOnlineSubsystemTencent;
class FUniqueNetIdRail;
struct FGetUserInviteTaskResult;
struct FOnlineError;

/**
 * 
 */
class FOnlineSessionTencentRail : public FOnlineSessionTencent
{
private:
	/** Hidden on purpose */
	FOnlineSessionTencentRail() = delete;

public:

	// ~Begin IOnlineSession Interface
	virtual bool CreateSession(int32 HostingPlayerNum, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) override;
	virtual bool CreateSession(const FUniqueNetId& HostingPlayerId, FName SessionName, const FOnlineSessionSettings& NewSessionSettings) override;
	virtual bool StartSession(FName SessionName) override;
	virtual bool UpdateSession(FName SessionName, FOnlineSessionSettings& UpdatedSessionSettings, bool bShouldRefreshOnlineData = true) override;
	virtual bool EndSession(FName SessionName) override;
	virtual bool DestroySession(FName SessionName, const FOnDestroySessionCompleteDelegate& CompletionDelegate = FOnDestroySessionCompleteDelegate()) override;
	virtual bool JoinSession(int32 PlayerNum, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) override;
	virtual bool JoinSession(const FUniqueNetId& PlayerId, FName SessionName, const FOnlineSessionSearchResult& DesiredSession) override;
	virtual bool FindFriendSession(int32 LocalUserNum, const FUniqueNetId& Friend) override;
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const FUniqueNetId& Friend) override;
	virtual bool FindFriendSession(const FUniqueNetId& LocalUserId, const TArray<FUniqueNetIdRef>& FriendList) override;
	virtual bool SendSessionInviteToFriend(int32 LocalUserNum, FName SessionName, const FUniqueNetId& Friend) override;
	virtual bool SendSessionInviteToFriend(const FUniqueNetId& LocalUserId, FName SessionName, const FUniqueNetId& Friend) override;
	virtual bool SendSessionInviteToFriends(int32 LocalUserNum, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;
	virtual bool SendSessionInviteToFriends(const FUniqueNetId& LocalUserId, FName SessionName, const TArray< FUniqueNetIdRef >& Friends) override;
	virtual bool RegisterPlayer(FName SessionName, const FUniqueNetId& PlayerId, bool bWasInvited) override;
	virtual bool RegisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players, bool bWasInvited = false) override;
	virtual bool UnregisterPlayer(FName SessionName, const FUniqueNetId& PlayerId) override;
	virtual bool UnregisterPlayers(FName SessionName, const TArray< FUniqueNetIdRef >& Players) override;
	virtual void RegisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnRegisterLocalPlayerCompleteDelegate& Delegate) override;
	virtual void UnregisterLocalPlayer(const FUniqueNetId& PlayerId, FName SessionName, const FOnUnregisterLocalPlayerCompleteDelegate& Delegate) override;
	virtual void DumpSessionState() override;
	// ~End IOnlineSession Interface

	//~ Begin FOnlineSessionTencent Interface
	virtual bool Init() override;
	virtual void Tick(float DeltaTime) override;
	virtual void Shutdown() override;
	//~ End FOnlineSessionTencent Interface

PACKAGE_SCOPE:

	FOnlineSessionTencentRail(FOnlineSubsystemTencent* InSubsystem);
	virtual ~FOnlineSessionTencentRail();

	/** Any invite/join from the command line */
	struct FPendingInviteData
	{
		/** User invite information */
		FUniqueNetIdRailPtr InviterUserId;
		/** Server invite information */
		FString CommandLineArgs;
		/** Is this invite valid */
		bool bValidInvite;

		FPendingInviteData() :
			bValidInvite(false)
		{
		}
	};

	/** Contains information about a join/invite parsed from the commandline */
	FPendingInviteData PendingInvite;

	/** @return the last set of keys used to set session presence data */
	const TArray<FString>& GetCurrentPresenceKeys() const { return CurrentSessionPresenceKeys; }

	/**
	 * Parse a single search result out of a Rail invite (metadata/invite command line)
	 * and add the result to current search results
	 *
	 * @param InSearch search session to add result to
	 * @param InResult search result as metadata key/value pairs
	 */
	void ParseSearchResult(TSharedPtr<FOnlineSessionSearch> InSearch, const FGetUserInviteTaskResult& InResult);

	/**
	 * Retrieve the full details about a remote user session, returning the FOnlineSessionSearchResult via OnSessionUserInviteAccepted
	 *
	 * @param InLocalUser local user initiating the request
	 * @param InRemoteUser remote user to obtain session details for
	 */
	void QueryAcceptedUserInvitation(FUniqueNetIdRailRef InLocalUser, FUniqueNetIdRailRef InRemoteUser);

protected:

	/**
	 * Waits for the proper time to handle pending invites from the command line
	 */
	void TickPendingInvites(float DeltaTime);

	/**
	 * Parse the command line for pending session invites
	 */
	void CheckPendingSessionInvite();

	/**
	 * Registers all local players with the current session
	 *
	 * @param Session the session that they are registering in
	 */
	void RegisterLocalPlayers(FNamedOnlineSession* Session);

	/**
	 * Is the owner of this session local
	 *
	 * @param Session session of interest
	 *
	 * @return true if the owner is on this machine, false otherwise
	 */
	bool OwnsSession(FNamedOnlineSession* Session) const;

	/**
	 *	Create a game server session, advertised on the backend
	 *
	 * @param HostingPlayerNum local index of the user initiating the request
	 * @param Session newly allocated session to create
	 *
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 CreateInternetSession(int32 HostingPlayerNum, FNamedOnlineSession* Session);

	/**
	 * Delegate fired when the session keys have been updated for the CreateSession call
	 *
	 * @param SessionName name of session that was created
	 * @param Error result of the operation
	 */
	void OnCreateInternetSessionComplete(FName SessionName, const FOnlineError& Error);

	/**
	 * 	Join a game server session, advertised on the backend
	 *
	 * @param PlayerNum local index of the user initiating the request
	 * @param Session newly allocated session with join information
	 * @param SearchSession the desired session to join
	 *
	 *  @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 JoinInternetSession(int32 PlayerNum, FNamedOnlineSession* Session, const FOnlineSession* SearchSession);

	/**
	 * Delegate fired when the session keys have been updated for the JoinSession call
	 *
	 * @param SessionName name of session that was joined
	 * @param Error result of the operation
	 */
	void OnJoinInternetSessionComplete(FName SessionName, const FOnlineError& Error);

	/**
	 * Delegate fired when the session keys have been updated for an UpdateSession call
	 *
	 * @param SessionName name of session that was updated
	 * @param Error result of the operation
	 */
	void OnUpdateSessionComplete(FName SessionName, const FOnlineError& Error);

	/**
	 * Start an internet session, advertised on the backend
	 *
	 * @param Session session to end
	 *
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 StartInternetSession(FNamedOnlineSession* Session);

	/**
	 *	End an internet session, advertised on the backend
	 *
	 * @param Session session to end
	 *
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 EndInternetSession(FNamedOnlineSession* Session);

	/**
	 *	Destroy an internet session, advertised on the backend
	 *
	 * @param Session session to destroy
	 *
	 * @return ONLINE_SUCCESS if successful, an error code otherwise
	 */
	uint32 DestroyInternetSession(FNamedOnlineSession* Session, const FOnDestroySessionCompleteDelegate& CompletionDelegate);

	/**
	 * Delegate fired for each UpdateSessionMetadata call
	 *
	 * @param Error result of the operation
	 */
	DECLARE_DELEGATE_OneParam(FOnUpdateSessionMetadataComplete, const FOnlineError& /*Error*/);

	/**
	 * Represent a session as a series of key/value pairs
	 * Clears all previously known keys from the last call to this function
	 *
	 * @param InNamedSession session to represent via key/value pairs
	 */
	void UpdateSessionMetadata(const FNamedOnlineSession& InNamedSession, const FOnUpdateSessionMetadataComplete& InCompletionDelegate);

	/** Delegate fired when platform says friend metadata has changed */
	void OnFriendMetadataChangedEvent(const FUniqueNetId& UserId, const FMetadataPropertiesRail& Metadata);

	/** Last set of keys set by presence */
	TArray<FString> CurrentSessionPresenceKeys;
	/** Handle to friend metadata change events */
	FDelegateHandle OnFriendMetadataChangedDelegateHandle;

	friend class FOnlineAsyncTaskRailSetSessionMetadata;
};

typedef TSharedPtr<FOnlineSessionTencentRail, ESPMode::ThreadSafe> FOnlineSessionTencentRailPtr;


#endif // WITH_TENCENT_RAIL_SDK
