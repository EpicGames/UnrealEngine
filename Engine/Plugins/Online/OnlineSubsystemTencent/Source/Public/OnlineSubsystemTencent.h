// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TENCENTSDK

#include "OnlineSubsystemImpl.h"
#include "OnlineSubsystemTencentTypes.h" // IWYU pragma: keep
#include "OnlineSubsystemPackage.h"

/**
 * Delegate called when Tencent requires Anti-Addiction message to be displayed
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnAASDialog, const FString& /*DialogTitle*/, const FString& /*DialogText*/, const FString& /*ButtonText*/);
typedef FOnAASDialog::FDelegate FOnAASDialogDelegate;

#if WITH_TENCENT_RAIL_SDK
/** 
 * Delegate fired when platform says friend metadata has changed 
 *
 * @param UserId the remote user whose data has changed
 * @param Metadata the changed metadata
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnFriendMetadataChanged, const FUniqueNetId& /*UserId*/, const FMetadataPropertiesRail& /*Metadata*/);
typedef FOnFriendMetadataChanged::FDelegate FOnFriendMetadataChangedDelegate;
#endif // WITH_TENCENT_RAIL_SDK

/** Forward declarations of all interface classes */
typedef TSharedPtr<class FOnlineIdentityTencent, ESPMode::ThreadSafe> FOnlineIdentityTencentPtr;
typedef TSharedPtr<class FOnlineSessionTencent, ESPMode::ThreadSafe> FOnlineSessionTencentPtr;
typedef TSharedPtr<class FOnlineDirectoryTencent, ESPMode::ThreadSafe> FOnlineDirectoryTencentPtr;
typedef TSharedPtr<class FOnlineAsyncTaskManagerTencent, ESPMode::ThreadSafe> FOnlineAsyncTaskManagerTencentPtr;
#if WITH_TENCENT_RAIL_SDK
typedef TSharedPtr<class FOnlineFriendsTencent, ESPMode::ThreadSafe> FOnlineFriendsTencentPtr;
typedef TSharedPtr<class FOnlinePresenceTencent, ESPMode::ThreadSafe> FOnlinePresenceTencentPtr;
typedef TSharedPtr<class FOnlineExternalUITencent, ESPMode::ThreadSafe> FOnlineExternalUITencentPtr;
typedef TSharedPtr<class FOnlineUserTencent, ESPMode::ThreadSafe> FOnlineUserTencentPtr;
typedef TSharedPtr<class FMessageSanitizerTencent, ESPMode::ThreadSafe> FMessageSanitizerTencentPtr;
typedef TSharedPtr<class FOnlinePurchaseTencent, ESPMode::ThreadSafe> FOnlinePurchaseTencentPtr;
typedef TSharedPtr<class FOnlineStoreTencent, ESPMode::ThreadSafe> FOnlineStoreTencentPtr; 
#endif // WITH_TENCENT_RAIL_SDK

class FOnlineAsyncTaskManagerTencent;
class FOnlineAsyncTask;
class FOnlineAsyncItem;
class FRunnableThread;

/**
 * Tencent backend services
 */
class ONLINESUBSYSTEMTENCENT_API FOnlineSubsystemTencent
	: public FOnlineSubsystemImpl
	, public TSharedFromThis<FOnlineSubsystemTencent, ESPMode::ThreadSafe>
{
public:

	// IOnlineSubsystem

	virtual IOnlineSessionPtr GetSessionInterface() const override;
	virtual IOnlineFriendsPtr GetFriendsInterface() const override;
	virtual IOnlinePartyPtr GetPartyInterface() const override;
	virtual IOnlineGroupsPtr GetGroupsInterface() const override;
	virtual IOnlineSharedCloudPtr GetSharedCloudInterface() const override;
	virtual IOnlineUserCloudPtr GetUserCloudInterface() const override;
	virtual IOnlineEntitlementsPtr GetEntitlementsInterface() const override;
	virtual IOnlineLeaderboardsPtr GetLeaderboardsInterface() const override;
	virtual IOnlineVoicePtr GetVoiceInterface() const override;
	virtual IOnlineExternalUIPtr GetExternalUIInterface() const override;	
	virtual IOnlineTimePtr GetTimeInterface() const override;
	virtual IOnlineIdentityPtr GetIdentityInterface() const override;
	virtual IOnlineTitleFilePtr GetTitleFileInterface() const override;
	virtual IOnlineStoreV2Ptr GetStoreV2Interface() const override;
	virtual IOnlinePurchasePtr GetPurchaseInterface() const override;
	virtual IOnlineEventsPtr GetEventsInterface() const override;
	virtual IOnlineAchievementsPtr GetAchievementsInterface() const override;
	virtual IOnlineSharingPtr GetSharingInterface() const override;
	virtual IOnlineUserPtr GetUserInterface() const override;
	virtual IOnlineMessagePtr GetMessageInterface() const override;
	virtual IOnlinePresencePtr GetPresenceInterface() const override;
	virtual IOnlineChatPtr GetChatInterface() const override;
	virtual IOnlineStatsPtr GetStatsInterface() const override;
	virtual IOnlineTurnBasedPtr GetTurnBasedInterface() const override;
	virtual IOnlineTournamentPtr GetTournamentInterface() const override;
	virtual IMessageSanitizerPtr GetMessageSanitizer(int32 LocalUserNum, FString& OutAuthTypeToExclude) const override;
	virtual FText GetOnlineServiceName() const override;
	virtual FText GetSocialPlatformName() const override;

	virtual bool Init() override;
	virtual void PreUnload() override;
	virtual bool Shutdown() override;
	virtual FString GetAppId() const override;
	virtual bool Exec(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

	// FTSTickerObjectBase

	virtual bool Tick(float DeltaTime) override;

	// FOnlineSubsystemTencent

	virtual ~FOnlineSubsystemTencent();

	/**
	 * Delegate called when Anti Addiction dialog should be displayed
	 */
	DEFINE_ONLINE_DELEGATE_THREE_PARAM(OnAASDialog, const FString& /*DialogTitle*/, const FString& /*DialogText*/, const FString& /*ButtonText*/);

#if WITH_TENCENT_RAIL_SDK
	/** Delegate fired when platform says friend metadata has changed */
	DEFINE_ONLINE_DELEGATE_TWO_PARAM(OnFriendMetadataChanged, const FUniqueNetId& /*UserId*/, const FMetadataPropertiesRail& /*Metadata*/);
#endif // WITH_TENCENT_RAIL_SDK

PACKAGE_SCOPE:

	class FOnlineAsyncTaskManagerTencent* GetAsyncTaskManager() { return OnlineAsyncTaskThreadRunnable; }

	void QueueAsyncParallelTask(FOnlineAsyncTask* AsyncTask);
	void QueueAsyncTask(FOnlineAsyncTask* AsyncTask);
	void QueueAsyncOutgoingItem(FOnlineAsyncItem* AsyncItem);

#if WITH_TENCENT_RAIL_SDK
	/**
	 * Add metadata changes sent to RailSDK to a local cache to prevent unnecessary API calls
	 *
	 * @param InMetadata data in final form (has _s, _i, etc) known to be succesfully sent to the backend
	 */
	void AddToMetadataCache(const TMap<FString, FString>& InMetadata);
	/** @return local cache of metadata set on the RailSDK */
	const TMap<FString, FString>& GetMetadataCache() const { return MetadataDataCache; }
#endif // WITH_TENCENT_RAIL_SDK

	/** Only the factory makes instances */
	FOnlineSubsystemTencent() = delete;
	explicit FOnlineSubsystemTencent(FName InInstanceName);
	
private:

	bool HandleAuthExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleSessionExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandlePresenceExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleUsersExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleFriendExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);

	// Rail SDK exec commands
	bool HandleRailSdkWrapperExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	bool HandleRailSdkWrapperPlayerExecCommands(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar);
	
	void OnLoginChanged(int32 LocalUserNum);

	/**
	 * Check if we should be using the Rail SDK
	 * @return true if we should be using the Rail SDK
	 */
	bool UsesRailSdk() const;

	/** 
	 * Initialize Rail SDK 
	 *
	 * @return true if successful
	 */
	bool InitRailSdk();

	/**
	 * Shutdown Rail SDK
	 */
	void ShutdownRailSdk();

	/** Interface to the identity registration/auth services (TCLS login support) */
	FOnlineIdentityTencentPtr TencentIdentity;
	/** Interface to the session services (needed for TSS anticheat handling) */
	FOnlineSessionTencentPtr TencentSession;
#if WITH_TENCENT_RAIL_SDK
	/** Interface to the friends services */
	FOnlineFriendsTencentPtr TencentFriends;
	/** Interface to the presence services */
	FOnlinePresenceTencentPtr TencentPresence;
	/** Interface to the external UI */
	FOnlineExternalUITencentPtr TencentExternalUI;
	/** Interface to users lookup */
	FOnlineUserTencentPtr TencentUser;
	/** Interface to the message sanitizer */
	FMessageSanitizerTencentPtr TencentMessageSanitizer;
	/** Interface for purchasing */
	FOnlinePurchaseTencentPtr TencentPurchase;
	/** Interface for store */
	FOnlineStoreTencentPtr TencentStore;
	/** Game id for initializing Rail SDK */
	uint64 RailGameId;
	/** Map of all key/value pair data ever attempted to be set on the RailSDK */
	TMap<FString, FString> MetadataDataCache;
#endif //WITH_TENCENT_RAIL_SDK

	/** detect login changes from online identity and forward to other services */
	FDelegateHandle OnLoginChangedHandle;

	/** Online async task runnable */
	FOnlineAsyncTaskManagerTencent* OnlineAsyncTaskThreadRunnable;

	/** Online async task thread */
	FRunnableThread* OnlineAsyncTaskThread;
};

typedef TSharedPtr<FOnlineSubsystemTencent, ESPMode::ThreadSafe> FOnlineSubsystemTencentPtr;

#endif // WITH_TENCENTSDK
