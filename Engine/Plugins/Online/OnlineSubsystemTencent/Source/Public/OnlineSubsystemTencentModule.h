// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

/**
 * Module used for talking with an Tencent service via Http requests
 */
class FOnlineSubsystemTencentModule : public IModuleInterface
{
private:

	/** Class responsible for creating instance(s) of the subsystem */
	class FOnlineFactoryTencent* TencentFactory;

public:

	/**
	 * Constructor
	 */
	FOnlineSubsystemTencentModule() :
		TencentFactory(nullptr)
	{}

	/**
	 * Destructor
	 */
	virtual ~FOnlineSubsystemTencentModule() 
	{}

	// IModuleInterface

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override;

	/**
	 * Override this to set whether your module is allowed to be unloaded on the fly
	 *
	 * @return	Whether the module supports shutdown separate from the rest of the engine.
	 */
	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

	/**
	 * Override this to set whether your module would like cleanup on application shutdown
	 *
	 * @return	Whether the module supports shutdown on application exit
	 */
	virtual bool SupportsAutomaticShutdown() override
	{
		return false;
	}
};
