// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OnlineSubsystemTypes.h"
#include "Misc/Guid.h"
#include "OnlineSubsystemTencentPackage.h"

#if WITH_TENCENTSDK

#if WITH_TENCENT_RAIL_SDK

#include "OnlineKeyValuePair.h"
#include "RailSDK.h"

template <typename ValueType>
struct FRailIdKeyFuncs : public TDefaultMapKeyFuncs<rail::RailID, ValueType, false>
{
	static inline rail::RailID	GetSetKey(TPair<rail::RailID, ValueType> const& Element) { return Element.Key; }
	static inline uint32		GetKeyHash(rail::RailID const& Key) { return GetTypeHash(Key.get_id()); }
	static inline bool			Matches(rail::RailID const& A, rail::RailID const& B) { return A == B; }
};

template <typename ValueType>
using TRailIdMap = TMap<rail::RailID, ValueType, FDefaultSetAllocator, FRailIdKeyFuncs<ValueType>>;

/** Key/Value pairs stored per user on the Rail platform */
using FMetadataPropertiesRail = FOnlineKeyValuePairs<FString, FVariantData>;

using FUniqueNetIdRailRef = TSharedRef<const class FUniqueNetIdRail>;
using FUniqueNetIdRailPtr = TSharedPtr<const class FUniqueNetIdRail>;

/**
 * Rail specific implementation of the unique net id
 */
class FUniqueNetIdRail :
	public FUniqueNetId
{
PACKAGE_SCOPE:
	/** Holds the net id for a player */
	rail::RailID RailID;

	/** Hidden on purpose */
	FUniqueNetIdRail() = delete;

public:
	template<typename... TArgs>
	static FUniqueNetIdRailRef Create(TArgs&&... Args)
	{
		return MakeShareable(new FUniqueNetIdRail(Forward<TArgs>(Args)...));
	}

	virtual FName GetType() const override
	{
		return TENCENT_SUBSYSTEM;
	}

	/**
	 * Get the raw byte representation of this net id
	 * This data is platform dependent and shouldn't be manipulated directly
	 *
	 * @return byte array of size GetSize()
	 */
	virtual const uint8* GetBytes() const override
	{
		return (uint8*)&RailID;
	}

	/**
	 * Get the size of the id
	 *
	 * @return size in bytes of the id representation
	 */
	virtual int32 GetSize() const override
	{
		return sizeof(uint64);
	}

	/**
	 * Check the validity of the id
	 *
	 * @return true if this is a well formed ID, false otherwise
	 */
	virtual bool IsValid() const override
	{
		return RailID.IsValid();
	}

	/** global static instance of invalid (zero) id */
	static const FUniqueNetIdRef& EmptyId()
	{
		static const FUniqueNetIdRef EmptyId(Create(rail::kInvalidRailId));
		return EmptyId;
	}

	/**
	 * Platform specific conversion to string representation of data
	 *
	 * @return data in string form
	 */
	virtual FString ToString() const override
	{
		return FString::Printf(TEXT("%llu"), RailID.get_id());
	}

	/**
	 * Get a human readable representation of the net id
	 * Shouldn't be used for anything other than logging/debugging
	 *
	 * @return id in string form
	 */
	virtual FString ToDebugString() const override
	{
		if (RailID.IsValid())
		{
			const FString UniqueNetIdStr = FString::Printf(TEXT("[%llu] Domain: %d"), RailID.get_id(), static_cast<int32>(RailID.GetDomain()));
			return OSS_UNIQUEID_REDACT(*this, UniqueNetIdStr);
		}
		else
		{
			return FString::Printf(TEXT("INVALID [%llu]"), RailID.get_id());
		}
	}

	virtual uint32 GetTypeHash() const override
	{
		uint64 id = RailID.get_id();
		return ::GetTypeHash(id);
	}

	/** Convenience cast to RailID */
	operator rail::RailID() const
	{
		return RailID;
	}

private:
	/**
	 * Copy Constructor
	 *
	 * @param Src the id to copy
	 */
	explicit FUniqueNetIdRail(const FUniqueNetIdRail& Src) :
		RailID(Src.RailID)
	{
	}

	/**
	 * Constructs this object with the specified net id
	 *
	 * @param InUniqueNetId the id to set ours to
	 */
	explicit FUniqueNetIdRail(uint64 InUniqueNetId) :
		RailID(InUniqueNetId)
	{
	}

	/**
	 * Constructs this object with the RailID
	 *
	 * @param InUniqueNetId the id to set ours to
	 */
	explicit FUniqueNetIdRail(rail::RailID InRailID) :
		RailID(InRailID)
	{
	}

	/**
	 * Constructs this object with the specified net id
	 *
	 * @param String textual representation of an id
	 */
	explicit FUniqueNetIdRail(const FString& Str) :
		RailID(FCString::Atoi64(*Str))
	{
	}


	/**
	 * Constructs this object with the specified net id
	 *
	 * @param InUniqueNetId the id to set ours to (assumed to be FUniqueNetIdRail in fact)
	 */
	explicit FUniqueNetIdRail(const FUniqueNetId& InUniqueNetId) :
		RailID(*(uint64*)InUniqueNetId.GetBytes())
	{
	}
};

#endif // WITH_TENCENT_RAIL_SDK

class FOnlineSessionInfoTencent : public FOnlineSessionInfo
{
private:
	/** Hidden on purpose */
	FOnlineSessionInfoTencent(const FOnlineSessionInfoTencent& Src) = delete;
	FOnlineSessionInfoTencent& operator=(const FOnlineSessionInfoTencent& Src) = delete;


PACKAGE_SCOPE:

	/** UniqueId assigned to this session */
	FUniqueNetIdStringPtr SessionId;

public:

	/** Default constructor (for LAN and soon to be serialized values) */
	FOnlineSessionInfoTencent();
	/** Constructor for sessions that represent an advertised session */
	FOnlineSessionInfoTencent(const FUniqueNetIdStringPtr& InSessionId);

	~FOnlineSessionInfoTencent() = default;

	/**
	 * Initialize a session info with the address of this machine
	 */
	void Init();

	/**
	 * Checks if a session id is valid
	 *
	 * @param InSessionId the session id to check
	 * @return true if InSessionId is a valid session id, false if not
	 */
	static bool IsSessionIdValid(const FString& InSessionId)
	{
		if (!InSessionId.IsEmpty())
		{
			FGuid DummyGuid;
			bool bIsValid = FGuid::Parse(InSessionId, DummyGuid);
			return bIsValid;
		}
		return false;
	}

	virtual const uint8* GetBytes() const override
	{
		return nullptr;
	}

	virtual int32 GetSize() const override
	{
		return sizeof(FUniqueNetIdStringPtr);
	}

	virtual bool IsValid() const override
	{
		return SessionId.IsValid() && IsSessionIdValid(SessionId->UniqueNetIdStr);
	}

	virtual FString ToString() const override
	{
		return SessionId.IsValid() ? SessionId->ToString() : TEXT("INVALID");
	}

	virtual FString ToDebugString() const override
	{
		return FString::Printf(TEXT("SessionId: %s"),
			SessionId.IsValid() ? *SessionId->ToDebugString() : TEXT("INVALID"));
	}

	virtual const FUniqueNetId& GetSessionId() const override
	{
		static const FUniqueNetIdStringRef InvalidId = FUniqueNetIdString::Create(TEXT("Invalid"), TENCENT_SUBSYSTEM);
		return SessionId.IsValid() ? *SessionId : *InvalidId;
	}
};

#endif //WITH_TENCENTSDK
